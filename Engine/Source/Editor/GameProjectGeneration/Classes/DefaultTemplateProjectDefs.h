// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "TemplateProjectDefs.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "DefaultTemplateProjectDefs.generated.h"

#define UE_API GAMEPROJECTGENERATION_API

class UObject;

UCLASS(MinimalAPI)
class UDefaultTemplateProjectDefs : public UTemplateProjectDefs
{
	GENERATED_UCLASS_BODY()

	UE_API virtual bool GeneratesCode(const FString& ProjectTemplatePath) const override;

	UE_API virtual bool IsClassRename(const FString& DestFilename, const FString& SrcFilename, const FString& FileExtension) const override;

	UE_API virtual void AddConfigValues(TArray<FTemplateConfigValue>& ConfigValuesToSet, const FString& TemplateName, const FString& ProjectName, bool bShouldGenerateCode) const override;
};

#undef UE_API
