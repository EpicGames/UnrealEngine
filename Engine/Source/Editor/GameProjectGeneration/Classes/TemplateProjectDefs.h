// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "FeaturePackContentSource.h"
#include "Internationalization/PolyglotTextData.h"
#include "Internationalization/Text.h"
#include "Misc/AssertionMacros.h"
#include "Misc/CoreMiscDefines.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "TemplateProjectDefs.generated.h"

// does not require reflection exposure
struct FTemplateConfigValue
{
	FString ConfigFile;
	FString ConfigSection;
	FString ConfigKey;
	FString ConfigValue;
	bool bShouldReplaceExistingValue;

	GAMEPROJECTGENERATION_API FTemplateConfigValue(const FString& InFile, const FString& InSection, const FString& InKey, const FString& InValue, bool InShouldReplaceExistingValue);
};

USTRUCT()
struct FTemplateReplacement
{
	GENERATED_BODY()

	UPROPERTY()
	TArray<FString> Extensions;

	UPROPERTY()
	FString From;

	UPROPERTY()
	FString To;

	UPROPERTY()
	bool bCaseSensitive{false};
};

USTRUCT()
struct FTemplateFolderRename
{
	GENERATED_BODY()

	UPROPERTY()
	FString From;

	UPROPERTY()
	FString To;
};

USTRUCT()
struct FLocalizedTemplateString
{
	GENERATED_BODY()

	UPROPERTY()
	FString Language;

	UPROPERTY()
	FString Text;

	/** Find a text value for the current locale (or fallback to English) from the given array of localized strings. */
	static FText GetLocalizedText(const TArray<FLocalizedTemplateString>& LocalizedStrings);
};

USTRUCT()
struct FTemplateVariant
{
	GENERATED_BODY()
	
	UPROPERTY(config)
	FName Name;
	
	UPROPERTY(config)
	TArray<FLocalizedTemplateString> LocalizedDisplayNames;

	UPROPERTY(config)
	TArray<FLocalizedTemplateString> LocalizedDescriptions;

	/* Shared feature packs. The files in these packs listed in these structures marked as 'additionalfiles' will be copied on project generation*/
	UPROPERTY(config)
	TArray<FFeaturePackLevelSet> SharedContentPacks;

	/** Returns the display name for the current culture, or English if the current culture has no translation */
	FText GetDisplayNameText() const;

	/** Returns the display name for the current culture, or English if the current culture has no translation */
	FText GetLocalizedDescription() const;
};

UENUM()
enum class ETemplateSetting
{
	Languages,
	HardwareTarget,
	GraphicsPreset,
	StarterContent UE_DEPRECATED(5.6, "Ability to add Starter Content has been removed"),
	XR,
	Raytracing_DEPRECATED,
	Variants,
	All
};

UCLASS(abstract,config=TemplateDefs,MinimalAPI)
class UTemplateProjectDefs : public UObject
{
public:
	GENERATED_UCLASS_BODY()

	UPROPERTY(config)
	TArray<FLocalizedTemplateString> LocalizedDisplayNames;

	UPROPERTY(config)
	TArray<FLocalizedTemplateString> LocalizedDescriptions;

	UPROPERTY(config)
	TArray<FString> FoldersToIgnore;

	UPROPERTY(config)
	TArray<FString> FilesToIgnore;

	UPROPERTY(config)
	TArray<FTemplateFolderRename> FolderRenames;

	UPROPERTY(config)
	TArray<FTemplateReplacement> FilenameReplacements;

	UPROPERTY(config)
	TArray<FTemplateReplacement> ReplacementsInFiles;

	UPROPERTY(config)
	FString SortKey;

	UPROPERTY(config)
	TArray<FName> Categories;

	UPROPERTY(config)
	FString ClassTypes;

	UPROPERTY(config)
	FString AssetTypes;

	/* Should we allow creation of a project from this template. If this is false, the template is treated as a feature pack. */
	UPROPERTY(config)
	bool bAllowProjectCreation;

	/** Is this an enterprise template? */
	UPROPERTY(config)
	bool bIsEnterprise;

	/** Is this a blank template? Determines whether we can override the default map when creating the project. */
	UPROPERTY(config)
	bool bIsBlank;

	/** Is there a rendered thumbnail that should be treated as the project template icon. If this is true the thumbnail takes up the full tile size rather than a 64x64 icon */
	UPROPERTY(config)
	bool bThumbnailAsIcon;

	/* Optional list of settings to hide. If none are specified, then all settings are shown. */
	UPROPERTY(config)
	TArray<ETemplateSetting> HiddenSettings;
	
	/* Optional list of feature packs to include */
	UPROPERTY(config)
	TArray<FString> PacksToInclude;

	/** What detail level to edit when editing shared template resources  */
	UPROPERTY(config)
	EFeaturePackDetailLevel EditDetailLevelPreference;

	/* Shared feature packs. The files in these packs listed in these structures marked as 'additionalfiles' will be copied on project generation*/
	UPROPERTY(config)
	TArray<FFeaturePackLevelSet> SharedContentPacks;

#if WITH_EDITORONLY_DATA
	UE_DEPRECATED(5.6, "Ability to add Starter Content has been removed")
	UPROPERTY(config, meta = (DeprecatedProperty, DeprecationMessage = "Ability to add Starter Content has been removed"))
	FString StarterContent_DEPRECATED;
#endif // WITH_EDITORONLY_DATA

	/** List of variants for this template. Using an Array instead of a Map purely for ease of writing the ini file (allowing one variant per line) */
	UPROPERTY(config)
	TArray<FTemplateVariant> Variants;

	/** Fixes up all strings in this definitions object to replace \%TEMPLATENAME\% with the supplied template name and \%PROJECTNAME\% with the supplied project name */
	void FixupStrings(const FString& TemplateName, const FString& ProjectName);

	/** Returns the display name for the current culture, or English if the current culture has no translation */
	FText GetDisplayNameText();

	/** Returns the display name for the current culture, or English if the current culture has no translation */
	FText GetLocalizedDescription();

	/** Finds a Variant with the given name, returns null if not found */
	const FTemplateVariant* FindVariant(FName VariantName) const;

	/** Does this template generate C++ source? */
	virtual bool GeneratesCode(const FString& ProjectTemplatePath) const PURE_VIRTUAL(UTemplateProjectDefs::GeneratesCode, return false;)

	/** Callback for each file rename, so class renames can be extracted*/
	virtual bool IsClassRename(const FString& DestFilename, const FString& SrcFilename, const FString& FileExtension) const { return false; }

	/** Callback for adding config values */
	virtual void AddConfigValues(TArray<FTemplateConfigValue>& ConfigValuesToSet, const FString& TemplateName, const FString& ProjectName, bool bShouldGenerateCode) const { }

	/** Callback before project generation is done, allowing for custom project generation behavior */
	virtual bool PreGenerateProject(const FString& DestFolder, const FString& SrcFolder, const FString& NewProjectFile, const FString& TemplateFile, bool bShouldGenerateCode, FText& OutFailReason) { return true; }

	/** Callback after project generation is done, allowing for custom project generation behavior */
	virtual bool PostGenerateProject(const FString& DestFolder, const FString& SrcFolder, const FString& NewProjectFile, const FString& TemplateFile, bool bShouldGenerateCode, FText& OutFailReason) { return true;  }

private:
	void FixString(FString& InOutStringToFix, const FString& TemplateName, const FString& ProjectName);
};

USTRUCT()
struct FTemplateCategoryDef
{
	GENERATED_BODY()
	
	/** Key to use for identifying what category a template is in. */
	UPROPERTY()
	FName Key;

	/** Localized name for this template category. */
	UPROPERTY()
	TArray<FLocalizedTemplateString> LocalizedDisplayNames;

	/** Localized description for this template category. */
	UPROPERTY()
	TArray<FLocalizedTemplateString> LocalizedDescriptions;

	/** Reference to an icon to display for this category. Should be around 300x100. */
	UPROPERTY()
	FString Icon;

	/** Is this a major top-level category? Major categories are displayed as full rows, eg. the Game category.*/
	UPROPERTY()
	bool IsMajorCategory=false;
};

UCLASS(config=TemplateCategories)
class UTemplateCategories : public UObject
{
public:
	GENERATED_BODY()

	/** Array of all categories specified in this location. */
	UPROPERTY(config)
	TArray<FTemplateCategoryDef> Categories;
};