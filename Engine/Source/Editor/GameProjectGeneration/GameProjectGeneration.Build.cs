// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

[SupportedTargetTypes(TargetType.Editor, TargetType.Program)]
public class GameProjectGeneration : ModuleRules
{
	public GameProjectGeneration(ReadOnlyTargetRules Target) : base(Target)
	{
		// Enable truncation warnings in this module
		CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Warning;

		PublicDependencyModuleNames.AddRange(
			new string[] {
				"HardwareTargeting",
			}
		);

		PrivateIncludePathModuleNames.AddRange(
			new string[] {
				"AssetRegistry",
				"ContentBrowser",
				"MainFrame",
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Analytics",
				"ApplicationCore",
				"AppFramework",
				"AssetTools",
				"ClassViewer",
				"Core",
				"CoreUObject",
				"Engine",
				"EngineSettings",
				"InputCore",
				"Projects",
				"RenderCore",
				"Slate",
				"SlateCore",
				"EditorWidgets",
				"ToolWidgets",
				"SourceControl",
 				"TargetPlatform",
				"EditorFramework",
				"EditorSubsystem",
				"UnrealEd",
				"DesktopPlatform",
				"LauncherPlatform",
				"AddContentDialog",
				"AudioMixer",
				"AudioMixerCore",
				"ContentBrowserData"
			}
		);

		DynamicallyLoadedModuleNames.AddRange(
			new string[] {
				"AssetRegistry",
				"ContentBrowser",
				"Documentation",
				"MainFrame",
			}
		);

		if(Target.bWithLiveCoding)
		{
			PrivateIncludePathModuleNames.Add("LiveCoding");
		}
	}
}
