// Copyright Epic Games, Inc. All Rights Reserved.
#include "ClassTemplateEditorSubsystem.h"

#include "Delegates/Delegate.h"
#include "Editor.h"
#include "Editor/EditorEngine.h"
#include "HAL/Platform.h"
#include "Interfaces/IPluginManager.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Misc/AssertionMacros.h"
#include "Misc/CoreDelegates.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "Templates/Casts.h"
#include "Templates/SharedPointer.h"
#include "UObject/Class.h"
#include "UObject/UObjectIterator.h"
#include "UObject/WeakObjectPtr.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ClassTemplateEditorSubsystem)

#define LOCTEXT_NAMESPACE "ClassTemplateSubsystem"


FString UClassTemplate::GetDirectory() const
{
	return UClassTemplateEditorSubsystem::GetEngineTemplateDirectory();
}

FString UClassTemplate::GetFilename() const
{
	if (GeneratedBaseClass)
	{
		return GeneratedBaseClass->GetName();
	}

	return *GetName();
}

FString UClassTemplate::GetHeaderFilename() const
{
	static const FString Extension = TEXT(".h.template");
	return GetFilename() + Extension;
}

FString UClassTemplate::GetSourceFilename() const
{
	static const FString Extension = TEXT(".cpp.template");
	return GetFilename() + Extension;
}

bool UClassTemplate::ReadHeader(FString& OutHeaderFileText, FText& OutFailReason) const
{
	const FString Dir = GetDirectory();
	const FString FilePath = Dir / GetHeaderFilename();

	if (!FFileHelper::LoadFileToString(OutHeaderFileText, *FilePath))
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("FilePath"), FText::FromString(FilePath));
		OutFailReason = FText::Format(LOCTEXT("ReadClassTemplateFailure_FailedToReadTemplateHeaderFile", "Failed to read template source file \"{FilePath}\""), Args);
		return false;
	}

	return true;
}

bool UClassTemplate::ReadSource(FString& OutSourceFileText, FText& OutFailReason) const
{
	const FString Dir = GetDirectory();
	const FString FilePath = Dir / GetSourceFilename();

	if (!FFileHelper::LoadFileToString(OutSourceFileText, *FilePath))
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("FilePath"), FText::FromString(FilePath));
		OutFailReason = FText::Format(LOCTEXT("ReadClassTemplateFailure_FailedToReadTemplateSourceFile", "Failed to read template source file \"{FilePath}\""), Args);
		return false;
	}

	return true;
}

void UClassTemplate::BeginDestroy()
{
	if (GEditor)
	{
		if (UClassTemplateEditorSubsystem* TemplateSubsystem = GEditor->GetEditorSubsystem<UClassTemplateEditorSubsystem>())
		{
			TemplateSubsystem->Unregister(this);
		}
	}

	Super::BeginDestroy();
}

const UClass* UClassTemplate::GetGeneratedBaseClass() const
{
	return GeneratedBaseClass;
}

void UClassTemplate::SetGeneratedBaseClass(UClass* InClass)
{
	GeneratedBaseClass = InClass;

	const UClass* Class = GetClass();
	check(Class);

	const bool bIsAbstract = Class->HasAnyClassFlags(CLASS_Abstract);
	if (GEditor && !bIsAbstract)
	{
		if (UClassTemplateEditorSubsystem* TemplateSubsystem = GEditor->GetEditorSubsystem<UClassTemplateEditorSubsystem>())
		{
			if (TemplateSubsystem->ContainsClassTemplate(GeneratedBaseClass))
			{
				TemplateSubsystem->Unregister(this);
			}

			TemplateSubsystem->Register(this);
		}
	}
}

FString UPluginClassTemplate::GetDirectory() const
{
	TSharedPtr<IPlugin> Plugin = IPluginManager::Get().FindPlugin(PluginName);
	if (Plugin.IsValid())
	{
		return Plugin->GetContentDir() / TEXT("Templates");
	}

	return UClassTemplate::GetDirectory();
}

UClassTemplateEditorSubsystem::UClassTemplateEditorSubsystem()
	: UEditorSubsystem()
{
	FCoreDelegates::GetOnPostEngineInit().AddUObject(this, &UClassTemplateEditorSubsystem::RegisterTemplates);
}

void UClassTemplateEditorSubsystem::Deinitialize()
{
	TemplateRegistry.Reset();
	FCoreDelegates::GetOnPostEngineInit().RemoveAll(this);
}

void UClassTemplateEditorSubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
	RegisterTemplates();
}

void UClassTemplateEditorSubsystem::RegisterTemplates()
{
	for (TObjectIterator<UClass> It; It; ++It)
	{
		if (It->IsChildOf(UClassTemplate::StaticClass()) && !It->HasAnyClassFlags(CLASS_Abstract))
		{
			const UClassTemplate* ClassTemplate = Cast<const UClassTemplate>((*It)->GetDefaultObject());
			Register(ClassTemplate);
		}
	}
}

void UClassTemplateEditorSubsystem::Register(const UClassTemplate* InClassTemplate)
{
	if (!ensure(InClassTemplate))
	{
		return;
	}

	TWeakObjectPtr<const UClassTemplate> ClassTemplate = InClassTemplate;
	TWeakObjectPtr<const UClass> GeneratedBaseClass = ClassTemplate->GetGeneratedBaseClass();
	if (ensure(GeneratedBaseClass.IsValid()))
	{
		const bool bIsRegistered = TemplateRegistry.Contains(GeneratedBaseClass);
		if (!bIsRegistered)
		{
			TemplateRegistry.Add(GeneratedBaseClass, ClassTemplate);
		}
	}
}

bool UClassTemplateEditorSubsystem::Unregister(const UClassTemplate* InClassTemplate)
{
	if (!ensure(InClassTemplate))
	{
		return false;
	}

	TWeakObjectPtr<const UClass> Class = InClassTemplate->GetGeneratedBaseClass();
	return TemplateRegistry.Remove(Class) > 0;
}

bool UClassTemplateEditorSubsystem::ContainsClassTemplate(const UClass* InClass) const
{
	const TWeakObjectPtr<const UClass> ClassPtr = MakeWeakObjectPtr(InClass);
	return TemplateRegistry.Contains(ClassPtr);
}

const UClassTemplate* UClassTemplateEditorSubsystem::FindClassTemplate(const UClass* InClass) const
{
	const TWeakObjectPtr<const UClass> ClassPtr = MakeWeakObjectPtr(InClass);
	return TemplateRegistry.FindRef(ClassPtr).Get();
}

FString UClassTemplateEditorSubsystem::GetEngineTemplateDirectory()
{
	return FPaths::EngineContentDir() / TEXT("Editor") / TEXT("Templates");
}
#undef LOCTEXT_NAMESPACE
