// Copyright Epic Games, Inc. All Rights Reserved.

#include "DefaultTemplateProjectDefs.h"

#include "HAL/FileManager.h"
#include "HAL/Platform.h"
#include "Misc/AssertionMacros.h"
#include "Misc/CString.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DefaultTemplateProjectDefs)

UDefaultTemplateProjectDefs::UDefaultTemplateProjectDefs(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

bool UDefaultTemplateProjectDefs::GeneratesCode(const FString& ProjectTemplatePath) const
{
	// Only generate code if the template has a source folder
	const FString SourceFolder = (ProjectTemplatePath / TEXT("Source"));
	return IFileManager::Get().DirectoryExists(*SourceFolder);
}

bool UDefaultTemplateProjectDefs::IsClassRename(const FString& DestFilename, const FString& SrcFilename, const FString& FileExtension) const
{
	// we shouldn't be getting this call if it's a file who's name didn't change
	check(FPaths::GetBaseFilename(SrcFilename) != FPaths::GetBaseFilename(DestFilename));

	// look for headers
	if (FileExtension == TEXT("h"))
	{
		FString FileContents;
		if (ensure(FFileHelper::LoadFileToString(FileContents, *DestFilename)))
		{
			// @todo uht: Checking file contents to see if this is a UObject class.  Sort of fragile here.
			if (FileContents.Contains(TEXT(".generated.h\""), ESearchCase::IgnoreCase))
			{
				return true;
			}
		}
	}

	return false;
}

void UDefaultTemplateProjectDefs::AddConfigValues(TArray<FTemplateConfigValue>& ConfigValuesToSet, const FString& TemplateName, const FString& ProjectName, bool bShouldGenerateCode) const
{
	const FString ActiveGameNameRedirectsValue_LongName = FString::Printf(TEXT("(OldGameName=\"/Script/%s\",NewGameName=\"/Script/%s\")"), *TemplateName, *ProjectName);
	const FString ActiveGameNameRedirectsValue_ShortName = FString::Printf(TEXT("(OldGameName=\"%s\",NewGameName=\"/Script/%s\")"), *TemplateName, *ProjectName);
	new (ConfigValuesToSet) FTemplateConfigValue(TEXT("DefaultEngine.ini"), TEXT("/Script/Engine.Engine"), TEXT("+ActiveGameNameRedirects"), *ActiveGameNameRedirectsValue_LongName, /*InShouldReplaceExistingValue=*/false);
	new (ConfigValuesToSet) FTemplateConfigValue(TEXT("DefaultEngine.ini"), TEXT("/Script/Engine.Engine"), TEXT("+ActiveGameNameRedirects"), *ActiveGameNameRedirectsValue_ShortName, /*InShouldReplaceExistingValue=*/false);
}
