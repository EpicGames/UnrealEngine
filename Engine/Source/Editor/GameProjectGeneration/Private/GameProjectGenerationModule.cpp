// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameProjectGenerationModule.h"

#include "Brushes/SlateDynamicImageBrush.h"
#include "Containers/EnumAsByte.h"
#include "GameProjectGenerationLog.h"
#include "GameProjectUtils.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformCrt.h"
#include "Interfaces/IPluginManager.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Logging/LogMacros.h"
#include "Math/Vector2D.h"
#include "Math/Vector4.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Paths.h"
#include "Misc/ScopedSlowTask.h"
#include "SNewClassDialog.h"
#include "SProjectBrowser.h"
#include "SProjectDialog.h"
#include "Styling/SlateBrush.h"
#include "TemplateCategory.h"
#include "TemplateProjectDefs.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

class UClass;
struct FModuleContextInfo;


IMPLEMENT_MODULE( FGameProjectGenerationModule, GameProjectGeneration );
DEFINE_LOG_CATEGORY(LogGameProjectGeneration);

#define LOCTEXT_NAMESPACE "GameProjectGeneration"

void FGameProjectGenerationModule::StartupModule()
{
	LoadTemplateCategories();
}

void FGameProjectGenerationModule::ShutdownModule()
{
	
}

TSharedRef<class SWidget> FGameProjectGenerationModule::CreateGameProjectDialog(bool bAllowProjectOpening, bool bAllowProjectCreate)
{
	ensure(bAllowProjectOpening || bAllowProjectCreate);

	EProjectDialogModeMode Mode = EProjectDialogModeMode::Hybrid;

	if (bAllowProjectOpening && !bAllowProjectCreate)
	{
		Mode = EProjectDialogModeMode::OpenProject;
	}
	else if (bAllowProjectCreate && !bAllowProjectOpening)
	{
		Mode = EProjectDialogModeMode::NewProject;
	}

	return SNew(SProjectDialog, Mode);
}


TSharedRef<class SWidget> FGameProjectGenerationModule::CreateNewClassDialog(const UClass* InClass)
{
	return SNew(SNewClassDialog).Class(InClass);
}

TSharedRef<SWidget> FGameProjectGenerationModule::CreateProjectBrowser(bool bInShowHeaderBar, bool bInAllowScrolling)
{
	return SNew(SProjectBrowser).ShowHeaderBar(bInShowHeaderBar).AllowScrolling(bInAllowScrolling);
}

TSharedRef<class SWidget> FGameProjectGenerationModule::CreateEmbeddedProjectBrowser()
{
	return SNew(SProjectBrowser).IsEmbedded(true);
}

void FGameProjectGenerationModule::OpenAddCodeToProjectDialog(const FAddToProjectConfig& Config)
{
	GameProjectUtils::OpenAddToProjectDialog(Config, EClassDomain::Native);
	AddCodeToProjectDialogOpenedEvent.Broadcast();
}

void FGameProjectGenerationModule::OpenAddBlueprintToProjectDialog(const FAddToProjectConfig& Config)
{
	GameProjectUtils::OpenAddToProjectDialog(Config, EClassDomain::Blueprint);
}

void FGameProjectGenerationModule::TryMakeProjectFileWriteable(const FString& ProjectFile)
{
	GameProjectUtils::TryMakeProjectFileWriteable(ProjectFile);
}

void FGameProjectGenerationModule::CheckForOutOfDateGameProjectFile()
{
	GameProjectUtils::CheckForOutOfDateGameProjectFile();
}


bool FGameProjectGenerationModule::UpdateGameProject(const FString& ProjectFile, const FString& EngineIdentifier, FText& OutFailReason)
{
	return GameProjectUtils::UpdateGameProject(ProjectFile, EngineIdentifier, OutFailReason);
}


bool FGameProjectGenerationModule::UpdateCodeProject(FText& OutFailReason, FText& OutFailLog)
{
	FScopedSlowTask SlowTask(0, LOCTEXT( "UpdatingCodeProject", "Updating code project..." ) );
	SlowTask.MakeDialog();

	return GameProjectUtils::GenerateCodeProjectFiles(FPaths::GetProjectFilePath(), OutFailReason, OutFailLog);
}

bool FGameProjectGenerationModule::GenerateBasicSourceCode(TArray<FString>& OutCreatedFiles, FText& OutFailReason)
{
	return GameProjectUtils::GenerateBasicSourceCode(OutCreatedFiles, OutFailReason);
}

bool FGameProjectGenerationModule::ProjectHasCodeFiles()
{
	return GameProjectUtils::ProjectHasCodeFiles();
}

FString FGameProjectGenerationModule::DetermineModuleIncludePath(const FModuleContextInfo& ModuleInfo, const FString& FileRelativeTo)
{
	return GameProjectUtils::DetermineModuleIncludePath(ModuleInfo, FileRelativeTo);
}

const TArray<FModuleContextInfo>& FGameProjectGenerationModule::GetCurrentProjectModules()
{
	return GameProjectUtils::GetCurrentProjectModules();
}

bool FGameProjectGenerationModule::IsValidBaseClassForCreation(const UClass* InClass, const FModuleContextInfo& InModuleInfo)
{
	return GameProjectUtils::IsValidBaseClassForCreation(InClass, InModuleInfo);
}

bool FGameProjectGenerationModule::IsValidBaseClassForCreation(const UClass* InClass, const TArray<FModuleContextInfo>& InModuleInfoArray)
{
	return GameProjectUtils::IsValidBaseClassForCreation(InClass, InModuleInfoArray);
}

void FGameProjectGenerationModule::GetProjectSourceDirectoryInfo(int32& OutNumFiles, int64& OutDirectorySize)
{
	GameProjectUtils::GetProjectSourceDirectoryInfo(OutNumFiles, OutDirectorySize);
}

void FGameProjectGenerationModule::CheckAndWarnProjectFilenameValid()
{
	GameProjectUtils::CheckAndWarnProjectFilenameValid();
}


void FGameProjectGenerationModule::UpdateSupportedTargetPlatforms(const FName& InPlatformName, const bool bIsSupported)
{
	GameProjectUtils::UpdateSupportedTargetPlatforms(InPlatformName, bIsSupported);
}

void FGameProjectGenerationModule::ClearSupportedTargetPlatforms()
{
	GameProjectUtils::ClearSupportedTargetPlatforms();
}

bool FGameProjectGenerationModule::CreateProject(const FProjectInformation& InProjectInfo, FText& OutFailReason, FText& OutFailLog, TArray<FString>* OutCreatedFiles)
{
	return GameProjectUtils::CreateProject(InProjectInfo, OutFailReason, OutFailLog, OutCreatedFiles);
}

bool FGameProjectGenerationModule::OpenProject(const FString& InProjectFile, FText& OutFailReason)
{
	return GameProjectUtils::OpenProject(InProjectFile, OutFailReason);
}

void FGameProjectGenerationModule::LoadTemplateCategories()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FGameProjectGenerationModule::LoadTemplateCategories);

	// Now discover and all data driven templates
	TArray<FString> TemplateRootFolders;

	// @todo rocket make template folder locations extensible.
	TemplateRootFolders.Add(FPaths::RootDir() / TEXT("Templates"));

	// Add the Enterprise templates
	TemplateRootFolders.Add(FPaths::EnterpriseDir() / TEXT("Templates"));

	// allow plugins to define templates
	TArray<TSharedRef<IPlugin>> Plugins = IPluginManager::Get().GetDiscoveredPlugins();
	for (const TSharedRef<IPlugin>& Plugin : Plugins)
	{
		FString PluginDirectory = Plugin->GetBaseDir();
		if (!PluginDirectory.IsEmpty())
		{
			const FString PluginTemplatesDirectory = PluginDirectory / TEXT("Templates");

			if (IFileManager::Get().DirectoryExists(*PluginTemplatesDirectory))
			{
				TemplateRootFolders.Add(PluginTemplatesDirectory);
			}
		}
	}

	for (const FString& Root : TemplateRootFolders)
	{
		UTemplateCategories* CategoryDefs = GameProjectUtils::LoadTemplateCategories(Root);
		if (CategoryDefs != nullptr)
		{
			for (const FTemplateCategoryDef& Category : CategoryDefs->Categories)
			{
				TSharedPtr<FTemplateCategory> TemplateCategory;

				TSharedPtr<FTemplateCategory>* Existing = TemplateCategories.Find(Category.Key);
				if (Existing == nullptr)
				{
					TemplateCategory = TemplateCategories.Add(Category.Key, MakeShareable(new FTemplateCategory));
				}
				else
				{
					TemplateCategory = *Existing;
				}

				TemplateCategory->Key = Category.Key;
				TemplateCategory->LocalizedDisplayNames = Category.LocalizedDisplayNames;
				TemplateCategory->LocalizedDescriptions = Category.LocalizedDescriptions;
				
				const FName BrushName(*Category.Icon);

				TSharedPtr<FSlateDynamicImageBrush> Brush = MakeShared<FSlateDynamicImageBrush>(BrushName, FAppStyle::Get().GetVector("ProjectBrowser.MajorCategory.Size"));
				Brush->OutlineSettings.CornerRadii = FVector4(4, 4, 4, 4);
				Brush->OutlineSettings.RoundingType = ESlateBrushRoundingType::FixedRadius;
				Brush->DrawAs = ESlateBrushDrawType::RoundedBox;

				TemplateCategory->Icon = Brush;
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE
