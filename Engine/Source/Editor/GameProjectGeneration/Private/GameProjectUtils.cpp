// Copyright Epic Games, Inc. All Rights Reserved.


#include "GameProjectUtils.h"
#include "Misc/Guid.h"
#include "UObject/Class.h"
#include "FeaturePackContentSource.h"
#include "TemplateProjectDefs.h"
#include "HAL/PlatformFileManager.h"
#include "Misc/MessageDialog.h"
#include "HAL/FileManager.h"
#include "Misc/CommandLine.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/ScopedSlowTask.h"
#include "Misc/App.h"
#include "Misc/EngineVersion.h"
#include "Misc/StringOutputDevice.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWindow.h"
#include "Framework/Application/SlateApplication.h"
#include "ClassTemplateEditorSubsystem.h"
#include "Components/ActorComponent.h"
#include "Components/SceneComponent.h"
#include "GameFramework/Actor.h"
#include "GameFramework/Pawn.h"
#include "Editor/EditorPerProjectUserSettings.h"
#include "ISourceControlOperation.h"
#include "SourceControlOperations.h"
#include "ISourceControlProvider.h"
#include "ISourceControlModule.h"
#include "GeneralProjectSettings.h"
#include "GameFramework/Character.h"
#include "Misc/FeedbackContext.h"
#include "UObject/UObjectHash.h"
#include "UObject/UObjectIterator.h"
#include "GameFramework/GameModeBase.h"
#include "UnrealEdMisc.h"
#include "PluginDescriptor.h"
#include "Interfaces/IPluginManager.h"
#include "ProjectDescriptor.h"
#include "Interfaces/IProjectManager.h"
#include "GameProjectGenerationLog.h"
#include "DefaultTemplateProjectDefs.h"
#include "SNewClassDialog.h"
#include "FeaturedClasses.inl"
#include "TemplateCategory.h"

#include "Features/IModularFeatures.h"

#include "Interfaces/IMainFrameModule.h"

#include "AnalyticsEventAttribute.h"
#include "Interfaces/IAnalyticsProvider.h"
#include "EngineAnalytics.h"

#include "DesktopPlatformModule.h"
#include "Interfaces/ITargetPlatform.h"
#include "Interfaces/ITargetPlatformManagerModule.h"

#include "Styling/SlateIconFinder.h"
#include "SourceCodeNavigation.h"

#include "Misc/UProjectInfo.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Misc/HotReloadInterface.h"

#include "Dialogs/SOutputLogDialog.h"

#include "IAudioExtensionPlugin.h"
#include "AudioPluginUtilities.h"
#include "Sound/AudioSettings.h"

#include "PlatformInfo.h"
#include "Blueprint/BlueprintSupport.h"
#include "Settings/ProjectPackagingSettings.h"
#include "Async/ParallelFor.h"
#include "Dialog/SCustomDialog.h"

#if WITH_LIVE_CODING
#include "ILiveCodingModule.h"
#endif

#define LOCTEXT_NAMESPACE "GameProjectUtils"

#define MAX_PROJECT_PATH_BUFFER_SPACE 130 // Leave a reasonable buffer of additional characters to account for files created in the content directory during or after project generation
#define MAX_PROJECT_NAME_LENGTH 20 // Enforce a reasonable project name length so the path is not too long for FPlatformMisc::GetMaxPathLength()

#define MAX_CLASS_NAME_LENGTH 32 // Enforce a reasonable class name length so the path is not too long for FPlatformMisc::GetMaxPathLength()

TWeakPtr<SNotificationItem> GameProjectUtils::UpdateGameProjectNotification = NULL;
TWeakPtr<SNotificationItem> GameProjectUtils::WarningProjectNameNotification = NULL;

constexpr const TCHAR GameProjectUtils::IncludePathFormatString[];

namespace
{
	/** Get the configuration values for enabling Lumen by default. */
	void AddLumenConfigValues(const FProjectInformation& InProjectInfo, TArray<FTemplateConfigValue>& ConfigValues, bool bDisableStaticLighting = true)
	{
		// In some cases we may want to leave static lighting enabled (e.g. VRTemplate needs it as of UE 5.5)
		if (bDisableStaticLighting)
		{
			// Disable static lighting support
			ConfigValues.Emplace(TEXT("DefaultEngine.ini"),
				TEXT("/Script/Engine.RendererSettings"),
				TEXT("r.AllowStaticLighting"),
				TEXT("0"),
				true /* ShouldReplaceExistingValue */);
		}

		// Required for Lumen's Software Ray Tracing support
		ConfigValues.Emplace(TEXT("DefaultEngine.ini"),
			TEXT("/Script/Engine.RendererSettings"),
			TEXT("r.GenerateMeshDistanceFields"),
			TEXT("True"),
			true /* ShouldReplaceExistingValue */);

		// Enable Lumen Global Illumination by default
		ConfigValues.Emplace(TEXT("DefaultEngine.ini"),
			TEXT("/Script/Engine.RendererSettings"),
			TEXT("r.DynamicGlobalIlluminationMethod"),
			TEXT("1"),
			true /* ShouldReplaceExistingValue */);

		// Enable Lumen Reflections by default
		ConfigValues.Emplace(TEXT("DefaultEngine.ini"),
			TEXT("/Script/Engine.RendererSettings"),
			TEXT("r.ReflectionMethod"),
			TEXT("1"),
			true /* ShouldReplaceExistingValue */);
	}

	void AddNewProjectDefaultShadowConfigValues(const FProjectInformation& InProjectInfo, TArray<FTemplateConfigValue>& ConfigValues)
	{
		// Enable support for virtual shadow maps by default for new projects
		ConfigValues.Emplace(TEXT("DefaultEngine.ini"),
			TEXT("/Script/Engine.RendererSettings"),
			TEXT("r.Shadow.Virtual.Enable"),
			TEXT("1"),
			true /* ShouldReplaceExistingValue */);
	}

	void AddPostProcessingConfigValues(const FProjectInformation& InProjectInfo, TArray<FTemplateConfigValue>& ConfigValues)
	{
		// Enable support for ExtendDefaultLuminanceRange by default for new projects
		ConfigValues.Emplace(TEXT("DefaultEngine.ini"),
			TEXT("/Script/Engine.RendererSettings"),
			TEXT("r.DefaultFeature.AutoExposure.ExtendDefaultLuminanceRange"),
			TEXT("True"),
			false /* ShouldReplaceExistingValue */);

		// Enable Local Exposure by default for new projects
		ConfigValues.Emplace(TEXT("DefaultEngine.ini"),
			TEXT("/Script/Engine.RendererSettings"),
			TEXT("r.DefaultFeature.LocalExposure.HighlightContrastScale"),
			TEXT("0.8"),
			false /* ShouldReplaceExistingValue */);

		// Enable Local Exposure by default for new projects
		ConfigValues.Emplace(TEXT("DefaultEngine.ini"),
			TEXT("/Script/Engine.RendererSettings"),
			TEXT("r.DefaultFeature.LocalExposure.ShadowContrastScale"),
			TEXT("0.8"),
			false /* ShouldReplaceExistingValue */);
	}

	/** Get the configuration values for raytracing if enabled. */
	void AddRaytracingConfigValues(const FProjectInformation& InProjectInfo, TArray<FTemplateConfigValue>& ConfigValues)
	{
		ConfigValues.Emplace(TEXT("DefaultEngine.ini"),
			TEXT("/Script/WindowsTargetPlatform.WindowsTargetSettings"),
			TEXT("DefaultGraphicsRHI"),
			TEXT("DefaultGraphicsRHI_DX12"),
			false /* ShouldReplaceExistingValue */);

		ConfigValues.Emplace(TEXT("DefaultEngine.ini"),
			TEXT("/Script/Engine.RendererSettings"),
			TEXT("r.SkinCache.CompileShaders"),
			TEXT("True"),
			false /* ShouldReplaceExistingValue */);

		ConfigValues.Emplace(TEXT("DefaultEngine.ini"),
			TEXT("/Script/Engine.RendererSettings"),
			TEXT("r.RayTracing"),
			TEXT("True"),
			false /* ShouldReplaceExistingValue */);

		ConfigValues.Emplace(TEXT("DefaultEngine.ini"),
			TEXT("/Script/Engine.RendererSettings"),
			TEXT("r.RayTracing.RayTracingProxies.ProjectEnabled"),
			TEXT("True"),
			false /* ShouldReplaceExistingValue */);
	}

	/** Get the configuration values for enabling Substrate by default. */
	void AddSubstrateConfigValues(const FProjectInformation& InProjectInfo, TArray<FTemplateConfigValue>& ConfigValues)
	{
		ConfigValues.Emplace(TEXT("DefaultEngine.ini"),
			TEXT("/Script/Engine.RendererSettings"),
			TEXT("r.Substrate"),
			TEXT("True"),
			false /* ShouldReplaceExistingValue */);

		ConfigValues.Emplace(TEXT("DefaultEngine.ini"),
			TEXT("/Script/Engine.RendererSettings"),
			TEXT("r.Substrate.ProjectGBufferFormat"),
			TEXT("0"),
			false /* ShouldReplaceExistingValue */);
	}

	/** Get the configuration values for enabling WorldPartition by default. */
	void AddWorldPartitionConfigValues(const FProjectInformation& InProjectInfo, TArray<FTemplateConfigValue>& ConfigValues)
	{
		if (InProjectInfo.bIsBlankTemplate)
		{
			ConfigValues.Emplace(TEXT("DefaultEngine.ini"),
				TEXT("/Script/WorldPartitionEditor.WorldPartitionEditorSettings"),
				TEXT("CommandletClass"),
				TEXT("Class'/Script/UnrealEd.WorldPartitionConvertCommandlet'"),
				true /* ShouldReplaceExistingValue */);
		}
	}

	void AddUserInterfaceEngineConfigValues(const FProjectInformation& InProjectInfo, TArray<FTemplateConfigValue>& ConfigValues)
	{
		if (InProjectInfo.bIsBlankTemplate)
		{
			ConfigValues.Emplace(TEXT("DefaultEngine.ini"),
				TEXT("/Script/Engine.UserInterfaceSettings"),
				TEXT("bAuthorizeAutomaticWidgetVariableCreation"),
				TEXT("False"),
				true /* ShouldReplaceExistingValue */);

			ConfigValues.Emplace(TEXT("DefaultEngine.ini"),
				TEXT("/Script/Engine.UserInterfaceSettings"),
				TEXT("FontDPIPreset"),
				TEXT("Standard"),
				true /* ShouldReplaceExistingValue */);

			ConfigValues.Emplace(TEXT("DefaultEngine.ini"),
				TEXT("/Script/Engine.UserInterfaceSettings"),
				TEXT("FontDPI"),
				TEXT("72"),
				true /* ShouldReplaceExistingValue */);
		}
	}

	void AddUserInterfaceGameConfigValues(const FProjectInformation& InProjectInfo, TArray<FTemplateConfigValue>& ConfigValues)
	{
		if (InProjectInfo.bIsBlankTemplate)
		{
			ConfigValues.Emplace(TEXT("DefaultGame.ini"),
				TEXT("/Script/CommonUI.CommonUISettings"),
				TEXT("CommonButtonAcceptKeyHandling"),
				TEXT("TriggerClick"),
				true /* ShouldReplaceExistingValue */);
		}

		ConfigValues.Emplace(TEXT("DefaultGame.ini"),
							 TEXT("ConsoleVariables"),
							 TEXT("CommonUI.CheckKeyboardFocusAndParentage"),
							 TEXT("1"),
							 true /* ShouldReplaceExistingValue */);

		ConfigValues.Emplace(TEXT("DefaultGame.ini"),
							 TEXT("ConsoleVariables"),
							 TEXT("CommonUI.DisallowUserFocusedWidgetForPendingFocusRecipient"),
							 TEXT("1"),
							 true /* ShouldReplaceExistingValue */);

		ConfigValues.Emplace(TEXT("DefaultGame.ini"),
							 TEXT("ConsoleVariables"),
							 TEXT("CommonUI.FallbackToDesiredOnAutoRestoreFailure"),
							 TEXT("1"),
							 true /* ShouldReplaceExistingValue */);
	}
} // namespace <>

FText FNewClassInfo::GetClassName() const
{
	switch(ClassType)
	{
	case EClassType::UObject:
		return BaseClass ? BaseClass->GetDisplayNameText() : FText::GetEmpty();

	case EClassType::EmptyCpp:
		return LOCTEXT("NoParentClass", "None");

	case EClassType::SlateWidget:
		return LOCTEXT("SlateWidgetParentClass", "Slate Widget");

	case EClassType::SlateWidgetStyle:
		return LOCTEXT("SlateWidgetStyleParentClass", "Slate Widget Style");

	case EClassType::UInterface:
		return LOCTEXT("UInterfaceParentClass", "Unreal Interface");

	default:
		break;
	}

	return FText::GetEmpty();
}

FText FNewClassInfo::GetClassDescription(const bool bFullDescription/* = true*/) const
{
	switch(ClassType)
	{
	case EClassType::UObject:
		{
			if(BaseClass)
			{
				FString ClassDescription = BaseClass->GetToolTipText(/*bShortTooltip=*/!bFullDescription).ToString();

				if(!bFullDescription)
				{
					int32 FullStopIndex = 0;
					if(ClassDescription.FindChar('.', FullStopIndex))
					{
						// Only show the first sentence so as not to clutter up the UI with a detailed description of implementation details
						ClassDescription.LeftInline(FullStopIndex + 1, EAllowShrinking::No);
					}

					// Strip out any new-lines in the description
					ClassDescription.ReplaceInline(TEXT("\n"), TEXT(" "), ESearchCase::CaseSensitive);
				}

				return FText::FromString(ClassDescription);
			}
		}
		break;

	case EClassType::EmptyCpp:
		return LOCTEXT("EmptyClassDescription", "An empty C++ class with a default constructor and destructor.");

	case EClassType::SlateWidget:
		return LOCTEXT("SlateWidgetClassDescription", "A custom Slate widget, deriving from SCompoundWidget.");

	case EClassType::SlateWidgetStyle:
		return LOCTEXT("SlateWidgetStyleClassDescription", "A custom Slate widget style, deriving from FSlateWidgetStyle, along with its associated UObject wrapper class.");

	case EClassType::UInterface:
		return LOCTEXT("UInterfaceClassDescription", "A UObject Interface class, to be implemented by other UObject-based classes.");

	default:
		break;
	}

	return FText::GetEmpty();
}

const FSlateBrush* FNewClassInfo::GetClassIcon() const
{
	// Safe to do even if BaseClass is null, since FindIconForClass will return the default icon
	return FSlateIconFinder::FindIconBrushForClass(BaseClass);
}

FString FNewClassInfo::GetClassPrefixCPP() const
{
	switch(ClassType)
	{
	case EClassType::UObject:
		return BaseClass ? BaseClass->GetPrefixCPP() : TEXT("U");

	case EClassType::EmptyCpp:
		return TEXT("F");

	case EClassType::SlateWidget:
		return TEXT("S");

	case EClassType::SlateWidgetStyle:
		return TEXT("F");

	case EClassType::UInterface:
		return TEXT("U");

	default:
		break;
	}
	return TEXT("");
}

FString FNewClassInfo::GetClassNameCPP() const
{
	switch(ClassType)
	{
	case EClassType::UObject:
		return BaseClass ? BaseClass->GetName() : TEXT("");

	case EClassType::EmptyCpp:
		return TEXT("");

	case EClassType::SlateWidget:
		return TEXT("CompoundWidget");

	case EClassType::SlateWidgetStyle:
		return TEXT("SlateWidgetStyle");

	case EClassType::UInterface:
		return TEXT("Interface");

	default:
		break;
	}
	return TEXT("");
}

FString FNewClassInfo::GetCleanClassName(const FString& ClassName) const
{
	FString CleanClassName = ClassName;

	switch(ClassType)
	{
	case EClassType::SlateWidgetStyle:
		{
			// Slate widget style classes always take the form FMyThingWidget, and UMyThingWidgetStyle
			// if our class ends with either Widget or WidgetStyle, we need to strip those out to avoid silly looking duplicates
			if(CleanClassName.EndsWith(TEXT("Style")))
			{
				CleanClassName.LeftChopInline(5, EAllowShrinking::No); // 5 for "Style"
			}
			if(CleanClassName.EndsWith(TEXT("Widget")))
			{
				CleanClassName.LeftChopInline(6, EAllowShrinking::No); // 6 for "Widget"
			}
		}
		break;

	default:
		break;
	}

	return CleanClassName;
}

FString FNewClassInfo::GetFinalClassName(const FString& ClassName) const
{
	const FString CleanClassName = GetCleanClassName(ClassName);

	switch(ClassType)
	{
	case EClassType::SlateWidgetStyle:
		return FString::Printf(TEXT("%sWidgetStyle"), *CleanClassName);

	default:
		break;
	}

	return CleanClassName;
}

bool FNewClassInfo::GetIncludePath(FString& OutIncludePath) const
{
	switch(ClassType)
	{
	case EClassType::UObject:
		if(BaseClass && BaseClass->HasMetaData(TEXT("IncludePath")))
		{
			OutIncludePath = BaseClass->GetMetaData(TEXT("IncludePath"));
			return true;
		}
		break;

	case EClassType::SlateWidget:
		OutIncludePath = "Widgets/SCompoundWidget.h";
		return true;

	case EClassType::SlateWidgetStyle:
		OutIncludePath = "Styling/SlateWidgetStyle.h";
		return true;

	default:
		break;
	}
	return false;
}

FString FNewClassInfo::GetBaseClassHeaderFilename() const
{
	FString IncludePath;

	switch (ClassType)
	{
	case EClassType::UObject:
		if (BaseClass)
		{
			FString ClassHeaderPath;
			if (FSourceCodeNavigation::FindClassHeaderPath(BaseClass, ClassHeaderPath) && IFileManager::Get().FileSize(*ClassHeaderPath) != INDEX_NONE)
			{
				return ClassHeaderPath;
			}
		}
		break;

	case EClassType::SlateWidget:
	case EClassType::SlateWidgetStyle:
		GetIncludePath(IncludePath);
		return FPaths::EngineDir() / TEXT("Source") / TEXT("Runtime") / TEXT("SlateCore") / TEXT("Public") / IncludePath;
	default:
		return FString();
	}

	return FString();
}

FString FNewClassInfo::GetHeaderFilename(const FString& ClassName) const
{
	const FString HeaderFilename = GetFinalClassName(ClassName) + TEXT(".h");

	switch(ClassType)
	{
	case EClassType::SlateWidget:
		return TEXT("S") + HeaderFilename;

	default:
		break;
	}

	return HeaderFilename;
}

FString FNewClassInfo::GetSourceFilename(const FString& ClassName) const
{
	const FString SourceFilename = GetFinalClassName(ClassName) + TEXT(".cpp");

	switch(ClassType)
	{
	case EClassType::SlateWidget:
		return TEXT("S") + SourceFilename;

	default:
		break;
	}

	return SourceFilename;
}

FString FNewClassInfo::GetHeaderTemplateFilename() const
{
	switch(ClassType)
	{
		case EClassType::UObject:
		{
			if (BaseClass != nullptr)
			{
				if ((BaseClass == UActorComponent::StaticClass()) || (BaseClass == USceneComponent::StaticClass()))
				{
					return TEXT("ActorComponentClass.h.template");
				}
				else if (BaseClass == AActor::StaticClass())
				{
					return TEXT("ActorClass.h.template");
				}
				else if (BaseClass == APawn::StaticClass())
				{
					return TEXT("PawnClass.h.template");
				}
				else if (BaseClass == ACharacter::StaticClass())
				{
					return TEXT("CharacterClass.h.template");
				}
			}
			// Some other non-actor, non-component UObject class
			return TEXT( "UObjectClass.h.template" );
		}

	case EClassType::EmptyCpp:
		return TEXT("EmptyClass.h.template");

	case EClassType::SlateWidget:
		return TEXT("SlateWidget.h.template");

	case EClassType::SlateWidgetStyle:
		return TEXT("SlateWidgetStyle.h.template");

	case EClassType::UInterface:
		return TEXT("InterfaceClass.h.template");

	default:
		break;
	}
	return TEXT("");
}

FString FNewClassInfo::GetSourceTemplateFilename() const
{
	switch(ClassType)
	{
		case EClassType::UObject:
			if (BaseClass != nullptr)
			{
				if ((BaseClass == UActorComponent::StaticClass()) || (BaseClass == USceneComponent::StaticClass()))
				{
					return TEXT("ActorComponentClass.cpp.template");
				}
				else if (BaseClass == AActor::StaticClass())
				{
					return TEXT("ActorClass.cpp.template");
				}
				else if (BaseClass == APawn::StaticClass())
				{
					return TEXT("PawnClass.cpp.template");
				}
				else if (BaseClass == ACharacter::StaticClass())
				{
					return TEXT("CharacterClass.cpp.template");
				}
			}
			// Some other non-actor, non-component UObject class
			return TEXT( "UObjectClass.cpp.template" );

	case EClassType::EmptyCpp:
		return TEXT("EmptyClass.cpp.template");

	case EClassType::SlateWidget:
		return TEXT("SlateWidget.cpp.template");

	case EClassType::SlateWidgetStyle:
		return TEXT("SlateWidgetStyle.cpp.template");

	case EClassType::UInterface:
		return TEXT("InterfaceClass.cpp.template");

	default:
		break;
	}
	return TEXT("");
}

bool GameProjectUtils::IsValidProjectFileForCreation(const FString& ProjectFile, FText& OutFailReason)
{
	const FString BaseProjectFile = FPaths::GetBaseFilename(ProjectFile);
	if ( FPaths::GetPath(ProjectFile).IsEmpty() )
	{
		OutFailReason = LOCTEXT( "NoProjectPath", "You must specify a path." );
		return false;
	}

	if ( BaseProjectFile.IsEmpty() )
	{
		OutFailReason = LOCTEXT( "NoProjectName", "You must specify a project name." );
		return false;
	}

	if ( BaseProjectFile.Contains(TEXT(" ")) )
	{
		OutFailReason = LOCTEXT( "ProjectNameContainsSpace", "Project names may not contain a space." );
		return false;
	}

	if ( !FChar::IsAlpha(BaseProjectFile[0]) )
	{
		OutFailReason = LOCTEXT( "ProjectNameMustBeginWithACharacter", "Project names must begin with an alphabetic character." );
		return false;
	}

	if ( BaseProjectFile.Len() > MAX_PROJECT_NAME_LENGTH )
	{
		FFormatNamedArguments Args;
		Args.Add( TEXT("MaxProjectNameLength"), MAX_PROJECT_NAME_LENGTH );
		OutFailReason = FText::Format( LOCTEXT( "ProjectNameTooLong", "Project names must not be longer than {MaxProjectNameLength} characters." ), Args );
		return false;
	}

	const int32 MaxProjectPathLength = FPlatformMisc::GetMaxPathLength() - MAX_PROJECT_PATH_BUFFER_SPACE;
	if ( FPaths::GetBaseFilename(ProjectFile, false).Len() > MaxProjectPathLength )
	{
		FFormatNamedArguments Args;
		Args.Add( TEXT("MaxProjectPathLength"), MaxProjectPathLength );
		OutFailReason = FText::Format( LOCTEXT( "ProjectPathTooLong", "A project's path must not be longer than {MaxProjectPathLength} characters." ), Args );
		return false;
	}

	if ( FPaths::GetExtension(ProjectFile) != FProjectDescriptor::GetExtension() )
	{
		FFormatNamedArguments Args;
		Args.Add( TEXT("ProjectFileExtension"), FText::FromString( FProjectDescriptor::GetExtension() ) );
		OutFailReason = FText::Format( LOCTEXT( "InvalidProjectFileExtension", "File extension is not {ProjectFileExtension}" ), Args );
		return false;
	}

	FString IllegalNameCharacters;
	if ( !NameContainsOnlyLegalCharacters(BaseProjectFile, IllegalNameCharacters) )
	{
		FFormatNamedArguments Args;
		Args.Add( TEXT("IllegalNameCharacters"), FText::FromString( IllegalNameCharacters ) );
		OutFailReason = FText::Format( LOCTEXT( "ProjectNameContainsIllegalCharacters", "Project names may not contain the following characters: {IllegalNameCharacters}" ), Args );
		return false;
	}

	if (NameMatchesPlatformModuleName(BaseProjectFile))
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("PlatformModuleName"), FText::FromString(BaseProjectFile));
		OutFailReason = FText::Format(LOCTEXT("ProjectNameConflictsWithPlatformModuleName", "Project name conflicts with a platform name: {PlatformModuleName}"), Args);
		return false;
	}

	if ( !FPaths::ValidatePath(FPaths::GetPath(ProjectFile), &OutFailReason) )
	{
		return false;
	}

	if ( ProjectFileExists(ProjectFile) )
	{
		OutFailReason = LOCTEXT( "ProjectFileAlreadyExists", "This project file already exists." );
		return false;
	}

	if ( FPaths::ConvertRelativePathToFull(FPaths::GetPath(ProjectFile)).StartsWith( FPaths::ConvertRelativePathToFull(FPaths::EngineDir())) )
	{
		OutFailReason = LOCTEXT( "ProjectFileCannotBeUnderEngineFolder", "Project cannot be saved under the Engine folder. Please choose a different directory." );
		return false;
	}

	if ( AnyProjectFilesExistInFolder(FPaths::GetPath(ProjectFile)) )
	{
		FFormatNamedArguments Args;
		Args.Add( TEXT("ProjectFileExtension"), FText::FromString( FProjectDescriptor::GetExtension() ) );
		OutFailReason = FText::Format( LOCTEXT( "AProjectFileAlreadyExistsAtLoction", "Another .{ProjectFileExtension} file already exists in the specified folder" ), Args );
		return false;
	}

	// Don't allow any files within target directory so we can safely delete everything on failure
	TArray<FString> ExistingFiles;
	IFileManager::Get().FindFiles(ExistingFiles, *(FPaths::GetPath(ProjectFile) / TEXT("*")), true, true);
	if (ExistingFiles.Num() > 0)
	{
		OutFailReason = LOCTEXT("ProjectFileCannotBeWithExistingFiles", "Project cannot be saved in a folder with existing files. Please choose a different directory/project name.");
		return false;
	}

	return true;
}

bool GameProjectUtils::OpenProject(const FString& ProjectFile, FText& OutFailReason)
{
	if ( ProjectFile.IsEmpty() )
	{
		OutFailReason = LOCTEXT( "NoProjectFileSpecified", "You must specify a project file." );
		return false;
	}

	const FString BaseProjectFile = FPaths::GetBaseFilename(ProjectFile);
	if ( BaseProjectFile.Contains(TEXT(" ")) )
	{
		OutFailReason = LOCTEXT( "ProjectNameContainsSpace", "Project names may not contain a space." );
		return false;
	}

	if ( !FChar::IsAlpha(BaseProjectFile[0]) )
	{
		OutFailReason = LOCTEXT( "ProjectNameMustBeginWithACharacter", "Project names must begin with an alphabetic character." );
		return false;
	}

	const int32 MaxProjectPathLength = FPlatformMisc::GetMaxPathLength() - MAX_PROJECT_PATH_BUFFER_SPACE;
	if ( FPaths::GetBaseFilename(ProjectFile, false).Len() > MaxProjectPathLength )
	{
		FFormatNamedArguments Args;
		Args.Add( TEXT("MaxProjectPathLength"), MaxProjectPathLength );
		OutFailReason = FText::Format( LOCTEXT( "ProjectPathTooLong", "A project's path must not be longer than {MaxProjectPathLength} characters." ), Args );
		return false;
	}

	if ( FPaths::GetExtension(ProjectFile) != FProjectDescriptor::GetExtension() )
	{
		FFormatNamedArguments Args;
		Args.Add( TEXT("ProjectFileExtension"), FText::FromString( FProjectDescriptor::GetExtension() ) );
		OutFailReason = FText::Format( LOCTEXT( "InvalidProjectFileExtension", "File extension is not {ProjectFileExtension}" ), Args );
		return false;
	}

	FString IllegalNameCharacters;
	if ( !NameContainsOnlyLegalCharacters(BaseProjectFile, IllegalNameCharacters) )
	{
		FFormatNamedArguments Args;
		Args.Add( TEXT("IllegalNameCharacters"), FText::FromString( IllegalNameCharacters ) );
		OutFailReason = FText::Format( LOCTEXT( "ProjectNameContainsIllegalCharacters", "Project names may not contain the following characters: {IllegalNameCharacters}" ), Args );
		return false;
	}

	if ( !FPaths::ValidatePath(FPaths::GetPath(ProjectFile), &OutFailReason) )
	{
		return false;
	}

	if ( !ProjectFileExists(ProjectFile) )
	{
		FFormatNamedArguments Args;
		Args.Add( TEXT("ProjectFile"), FText::FromString( ProjectFile ) );
		OutFailReason = FText::Format( LOCTEXT( "ProjectFileDoesNotExist", "{ProjectFile} does not exist." ), Args );
		return false;
	}

	if (FEngineAnalytics::IsAvailable())
	{
		FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.OpenProject"));
	}

	FUnrealEdMisc::Get().SwitchProject(ProjectFile, false);

	return true;
}

bool GameProjectUtils::OpenCodeIDE(const FString& ProjectFile, FText& OutFailReason)
{
	if ( ProjectFile.IsEmpty() )
	{
		OutFailReason = LOCTEXT( "NoProjectFileSpecified", "You must specify a project file." );
		return false;
	}

	// Check whether this project is a foreign project. Don't use the cached project dictionary; we may have just created a new project.
	FString SolutionFolder;
	FString SolutionFilenameWithoutExtension;
	if( FUProjectDictionary(FPaths::RootDir()).IsForeignProject(ProjectFile) )
	{
		SolutionFolder = IFileManager::Get().ConvertToAbsolutePathForExternalAppForRead(*FPaths::GetPath(ProjectFile));
		SolutionFilenameWithoutExtension = FPaths::GetBaseFilename(ProjectFile);
	}
	else
	{
		SolutionFolder = IFileManager::Get().ConvertToAbsolutePathForExternalAppForRead(*FPaths::RootDir());
		SolutionFilenameWithoutExtension = TEXT("UE5");
	}

	if (!FSourceCodeNavigation::OpenProjectSolution(FPaths::Combine(SolutionFolder, SolutionFilenameWithoutExtension)))
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("AccessorName"), FSourceCodeNavigation::GetSelectedSourceCodeIDE());
		OutFailReason = FText::Format(LOCTEXT("OpenCodeIDE_FailedToOpen", "Failed to open selected source code accessor '{AccessorName}'"), Args);
		return false;
	}

	return true;
}

bool GameProjectUtils::CreateProject(const FProjectInformation& InProjectInfo, FText& OutFailReason, FText& OutFailLog, TArray<FString>* OutCreatedFiles)
{
	if ( !IsValidProjectFileForCreation(InProjectInfo.ProjectFilename, OutFailReason) )
	{
		return false;
	}

	FScopedSlowTask SlowTask(0, LOCTEXT( "CreatingProjectStatus", "Creating project..." ));
	SlowTask.MakeDialog();

	TOptional<FGuid> ProjectID;
	FString TemplateName;
	if ( InProjectInfo.TemplateFile.IsEmpty() )
	{
		ProjectID = GenerateProjectFromScratch(InProjectInfo, OutFailReason, OutFailLog);
		TemplateName = InProjectInfo.bShouldGenerateCode ? TEXT("Basic Code") : TEXT("Blank");
	}
	else
	{
		ProjectID = CreateProjectFromTemplate(InProjectInfo, OutFailReason, OutFailLog, OutCreatedFiles);
		TemplateName = FPaths::GetBaseFilename(InProjectInfo.TemplateFile);
	}

	bool bProjectCreationSuccessful = ProjectID.IsSet();

	if (!bProjectCreationSuccessful && CleanupIsEnabled())
	{
		// Delete the new project folder
		const FString NewProjectFolder = FPaths::GetPath(InProjectInfo.ProjectFilename);
		IFileManager::Get().DeleteDirectory(*NewProjectFolder, /*RequireExists=*/false, /*Tree=*/true);
		if( OutCreatedFiles != nullptr )
		{
			OutCreatedFiles->Empty();
		}
	}

	if( FEngineAnalytics::IsAvailable() )
	{
		TArray<FAnalyticsEventAttribute> EventAttributes;
		EventAttributes.Add(FAnalyticsEventAttribute(TEXT("Template"), TemplateName));
		EventAttributes.Add(FAnalyticsEventAttribute(TEXT("Category"), InProjectInfo.TemplateCategory.ToString()));
		EventAttributes.Add(FAnalyticsEventAttribute(TEXT("ProjectType"), InProjectInfo.bShouldGenerateCode ? TEXT("C++ Code") : TEXT("Content Only")));
		EventAttributes.Add(FAnalyticsEventAttribute(TEXT("Outcome"), bProjectCreationSuccessful ? TEXT("Successful") : TEXT("Failed")));
		EventAttributes.Add(FAnalyticsEventAttribute(TEXT("ProjectID"), *(bProjectCreationSuccessful ? ProjectID.GetValue().ToString() : FString())));

		if (InProjectInfo.TargetedHardware.IsSet())
		{
			UEnum* HardwareClassEnum = StaticEnum<EHardwareClass>();
			if (HardwareClassEnum != nullptr)
			{
				EventAttributes.Add(FAnalyticsEventAttribute(TEXT("HardwareClass"), HardwareClassEnum->GetNameStringByValue(static_cast<int32>(InProjectInfo.TargetedHardware.GetValue()))));
			}
		}

		if (InProjectInfo.DefaultGraphicsPerformance.IsSet())
		{
			UEnum* GraphicsPresetEnum = StaticEnum<EGraphicsPreset>();
			if (GraphicsPresetEnum != nullptr)
			{
				EventAttributes.Add(FAnalyticsEventAttribute(TEXT("GraphicsPreset"), GraphicsPresetEnum->GetNameStringByValue(static_cast<int32>(InProjectInfo.DefaultGraphicsPerformance.GetValue()))));
			}
		}

		FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.NewProject.ProjectCreated"), EventAttributes);
	}

	return bProjectCreationSuccessful;
}

void GameProjectUtils::CheckForOutOfDateGameProjectFile()
{
	if (FPaths::IsProjectFilePathSet() && !IProjectManager::Get().IsSuppressingProjectFileWrite())
	{
		if (IProjectManager::Get().IsCurrentProjectDirty())
		{
			FText FailMessage;
			TryMakeProjectFileWriteable(FPaths::GetProjectFilePath());
			if (!IProjectManager::Get().SaveCurrentProjectToDisk(FailMessage))
			{
				FMessageDialog::Open(EAppMsgType::Ok, FailMessage);
			}
		}

		// Check if the project file is an older version
		FProjectStatus ProjectStatus;
		bool bRequiresUpdate = false;
		if (IProjectManager::Get().QueryStatusForCurrentProject(ProjectStatus))
		{
			if ( ProjectStatus.bRequiresUpdate )
			{
				bRequiresUpdate = true;
				UE_LOGF(LogGameProjectGeneration, Log, "Project %ls requires update. Determined from QueryStatusForCurrentProject", *ProjectStatus.Name);
			}
		}

		// Check if there are any referenced plugins that do not have a matching supported plugins list
		const FProjectDescriptor* Project = IProjectManager::Get().GetCurrentProject();
		TArray<FPluginReferenceDescriptor> NewPluginReferences = Project->Plugins;
		for(FPluginReferenceDescriptor& Reference: NewPluginReferences)
		{
			if(Reference.bEnabled)
			{
				TSharedPtr<IPlugin> Plugin = IPluginManager::Get().FindPlugin(Reference.Name);
				if(Plugin.IsValid())
				{
					const FPluginDescriptor& Descriptor = Plugin->GetDescriptor();
					if(Reference.MarketplaceURL != Descriptor.MarketplaceURL)
					{
						UE_LOGF(LogGameProjectGeneration, Log, "Project %ls requires update. Plugin %ls MarketplaceURL value in project descriptor (%ls) differs from value in plugin descriptor (%ls) ",
							*ProjectStatus.Name, *Plugin->GetName(), *Reference.MarketplaceURL, *Descriptor.MarketplaceURL);

						Reference.MarketplaceURL = Descriptor.MarketplaceURL;
						bRequiresUpdate = true;
					}
					if(Reference.SupportedTargetPlatforms != Descriptor.SupportedTargetPlatforms)
					{
						auto CombineStrings = [](const TArray<FString>& Strings) -> FString
							{
								TStringBuilder<64> Output;
								const int32 LastIndex = Strings.Num() - 1;
								for (int32 I = 0; I <= LastIndex; ++I)
								{
									Output += Strings[I];
									if (I != LastIndex)
									{
										Output += TEXT(", ");
									}
								}
								return FString(Output);
							};

						UE_LOGF(LogGameProjectGeneration, Log, "Project %ls requires update. Plugin %ls SupportedTargetPlatforms value in project descriptor (%ls) differs from value in plugin descriptor (%ls) ",
							*ProjectStatus.Name, *Plugin->GetName(), *CombineStrings(Reference.SupportedTargetPlatforms), *CombineStrings(Descriptor.SupportedTargetPlatforms));

						Reference.SupportedTargetPlatforms = Descriptor.SupportedTargetPlatforms;
						bRequiresUpdate = true;
					}
				}
			}
		}

		// If we have updates pending, show the prompt
		if (bRequiresUpdate)
		{
			FProjectDescriptorModifier ModifyProject = FProjectDescriptorModifier::CreateLambda(
				[NewPluginReferences](FProjectDescriptor& Descriptor) { Descriptor.Plugins = NewPluginReferences; return true; });

			FSimpleDelegate OnUpdateProjectConfirm = FSimpleDelegate::CreateLambda(
				[ModifyProject]() { UpdateProject_Impl(&ModifyProject); });

			const FText UpdateProjectText = LOCTEXT("UpdateProjectFilePrompt", "Project file is out of date. Would you like to update it?");
			const FText UpdateProjectConfirmText = LOCTEXT("UpdateProjectFileConfirm", "Update");
			const FText UpdateProjectCancelText = LOCTEXT("UpdateProjectFileCancel", "Not Now");


			/** Utility functions for notifications */
			struct FSuppressDialogOptions
			{
				static bool ShouldSuppressModal()
				{
					bool bSuppressNotification = false;
					GConfig->GetBool(TEXT("GameProjectUtils"), TEXT("SuppressUpdateProjectFilePromptNotification"), bSuppressNotification, GEditorPerProjectIni);
					return bSuppressNotification;
				}

				static ECheckBoxState GetDontAskAgainCheckBoxState()
				{
					// Check the config for any preferences			
					return ShouldSuppressModal() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
				}

				static void OnDontAskAgainCheckBoxStateChanged(ECheckBoxState NewState)
				{
					// If the user selects to not show this again, set that in the config so we know about it in between sessions
					const bool bSuppressNotification = (NewState == ECheckBoxState::Checked);
					GConfig->SetBool(TEXT("GameProjectUtils"), TEXT("SuppressUpdateProjectFilePromptNotification"), bSuppressNotification, GEditorPerProjectIni);
				}
			}; 

			// If the user has specified to suppress this pop up, then just early out and exit	
			if (FSuppressDialogOptions::ShouldSuppressModal())
			{
				return;
			}
			
			FNotificationInfo Info(UpdateProjectText);
			Info.ExpireDuration = 10.0f;
			Info.bFireAndForget = true;
			Info.bUseLargeFont = false;
			Info.bUseThrobber = false;
			Info.bUseSuccessFailIcons = false;
			Info.ButtonDetails.Add(FNotificationButtonInfo(UpdateProjectConfirmText, FText(), OnUpdateProjectConfirm));
			Info.ButtonDetails.Add(FNotificationButtonInfo(UpdateProjectCancelText, FText(), FSimpleDelegate::CreateStatic(&GameProjectUtils::OnUpdateProjectCancel)));

			// Add a "Don't show this again" option
			Info.CheckBoxState = TAttribute<ECheckBoxState>::Create(&FSuppressDialogOptions::GetDontAskAgainCheckBoxState);
			Info.CheckBoxStateChanged = FOnCheckStateChanged::CreateStatic(&FSuppressDialogOptions::OnDontAskAgainCheckBoxStateChanged);
			Info.CheckBoxText = LOCTEXT("DefaultCheckBoxMessage", "Don't show this again");
			
			if (UpdateGameProjectNotification.IsValid())
			{
				UpdateGameProjectNotification.Pin()->ExpireAndFadeout();
				UpdateGameProjectNotification.Reset();
			}

			UpdateGameProjectNotification = FSlateNotificationManager::Get().AddNotification(Info);

			if (UpdateGameProjectNotification.IsValid())
			{
				UpdateGameProjectNotification.Pin()->SetCompletionState(SNotificationItem::CS_Pending);
			}
		}
	}
}

void GameProjectUtils::CheckAndWarnProjectFilenameValid()
{
	const FString& LoadedProjectFilePath = FPaths::IsProjectFilePathSet() ? FPaths::GetProjectFilePath() : FString();
	if ( !LoadedProjectFilePath.IsEmpty() )
	{
		const FString BaseProjectFile = FPaths::GetBaseFilename(LoadedProjectFilePath);
		if ( BaseProjectFile.Len() > MAX_PROJECT_NAME_LENGTH )
		{
			FFormatNamedArguments Args;
			Args.Add( TEXT("MaxProjectNameLength"), MAX_PROJECT_NAME_LENGTH );
			const FText WarningReason = FText::Format( LOCTEXT( "WarnProjectNameTooLong", "Project names must not be longer than {MaxProjectNameLength} characters.\nYou might have problems saving or modifying a project with a longer name." ), Args );
			const FText WarningReasonOkText = LOCTEXT("WarningReasonOkText", "Ok");

			FNotificationInfo Info(WarningReason);
			Info.bFireAndForget = false;
			Info.bUseLargeFont = false;
			Info.bUseThrobber = false;
			Info.bUseSuccessFailIcons = false;
			Info.FadeOutDuration = 3.f;
			Info.ButtonDetails.Add(FNotificationButtonInfo(WarningReasonOkText, FText(), FSimpleDelegate::CreateStatic(&GameProjectUtils::OnWarningReasonOk)));

			if (WarningProjectNameNotification.IsValid())
			{
				WarningProjectNameNotification.Pin()->ExpireAndFadeout();
				WarningProjectNameNotification.Reset();
			}

			WarningProjectNameNotification = FSlateNotificationManager::Get().AddNotification(Info);

			if (WarningProjectNameNotification.IsValid())
			{
				WarningProjectNameNotification.Pin()->SetCompletionState(SNotificationItem::CS_Pending);
			}
		}
	}
}

void GameProjectUtils::OnWarningReasonOk()
{
	if ( WarningProjectNameNotification.IsValid() )
	{
		WarningProjectNameNotification.Pin()->SetCompletionState(SNotificationItem::CS_None);
		WarningProjectNameNotification.Pin()->ExpireAndFadeout();
		WarningProjectNameNotification.Reset();
	}
}

bool GameProjectUtils::UpdateStartupModuleNames(FProjectDescriptor& Descriptor, const TArray<FString>* StartupModuleNames)
{
	if (StartupModuleNames == nullptr)
	{
		return false;
	}

	// Replace the modules names, if specified
	Descriptor.Modules.Empty();
	for (int32 Idx = 0; Idx < StartupModuleNames->Num(); Idx++)
	{
		Descriptor.Modules.Add(FModuleDescriptor(*(*StartupModuleNames)[Idx]));
	}

	ResetCurrentProjectModulesCache();

	return true;
}

bool GameProjectUtils::UpdateRequiredAdditionalDependencies(FProjectDescriptor& Descriptor, TArray<FString>& RequiredDependencies, const FString& ModuleName)
{
	bool bNeedsUpdate = false;

	for (auto& ModuleDesc : Descriptor.Modules)
	{
		if (ModuleDesc.Name != *ModuleName)
		{
			continue;
		}

		for (const auto& RequiredDep : RequiredDependencies)
		{
			if (!ModuleDesc.AdditionalDependencies.Contains(RequiredDep))
			{
				ModuleDesc.AdditionalDependencies.Add(RequiredDep);
				bNeedsUpdate = true;
			}
		}
	}

	return bNeedsUpdate;
}

bool GameProjectUtils::UpdateGameProject(const FString& ProjectFile, const FString& EngineIdentifier, FText& OutFailReason)
{
	return UpdateGameProjectFile(ProjectFile, EngineIdentifier, OutFailReason);
}

void GameProjectUtils::OpenAddToProjectDialog(const FAddToProjectConfig& Config, EClassDomain InDomain)
{
	// If we've been given a class then we only show the second page of the dialog, so we can make the window smaller as that page doesn't have as much content
	const FVector2D WindowSize = (Config._ParentClass) ? (InDomain == EClassDomain::Blueprint) ? FVector2D(940, 480) : FVector2D(940, 380) : FVector2D(940, 540);

	FText WindowTitle = Config._WindowTitle;
	if (WindowTitle.IsEmpty())
	{
		WindowTitle = InDomain == EClassDomain::Native ? LOCTEXT("AddCodeWindowHeader_Native", "Add C++ Class") : LOCTEXT("AddCodeWindowHeader_Blueprint", "Add Blueprint Class");
	}

	TSharedRef<SCustomDialog> AddCodeWindow =
		SNew(SCustomDialog)
		.Title( WindowTitle )
		.ClientSize( WindowSize );

	TSharedRef<SNewClassDialog> NewClassDialog =
		SNew(SNewClassDialog)
		.ParentWindow(AddCodeWindow)
		.Class(Config._ParentClass)
		.ClassViewerFilter(Config._AllowableParents)
		.ClassDomain(InDomain)
		.FeaturedClasses(Config._FeaturedClasses)
		.InitialPath(Config._InitialPath)
		.OnAddedToProject( Config._OnAddedToProject )
		.DefaultClassPrefix( Config._DefaultClassPrefix )
		.DefaultClassName( Config._DefaultClassName )
		.SyncContentBrowserToNewClass(Config._SyncContentBrowserToNewClass);

	AddCodeWindow->SetContent( NewClassDialog );
	AddCodeWindow->SetOnCancelHotkeyPressed(FExecuteAction::CreateSP(NewClassDialog, &SNewClassDialog::CancelClicked));
	AddCodeWindow->SetOnConfirmHotkeyPressed(FExecuteAction::CreateSP(NewClassDialog, &SNewClassDialog::OnConfirmHotkeyPressed));
	AddCodeWindow->SetWidgetToFocusOnActivate(NewClassDialog->GetWidgetToFocusOnActivate());

	TSharedPtr<SWindow> ParentWindow = Config._ParentWindow;
	if (!ParentWindow.IsValid())
	{
		static const FName MainFrameModuleName = "MainFrame";
		IMainFrameModule& MainFrameModule = FModuleManager::LoadModuleChecked<IMainFrameModule>(MainFrameModuleName);
		ParentWindow = MainFrameModule.GetParentWindow();
	}

	if (Config._bModal)
	{
		FSlateApplication::Get().AddModalWindow(AddCodeWindow, ParentWindow);
	}
	else if (ParentWindow.IsValid())
	{
		FSlateApplication::Get().AddWindowAsNativeChild(AddCodeWindow, ParentWindow.ToSharedRef());
	}
	else
	{
		FSlateApplication::Get().AddWindow(AddCodeWindow);
	}
}

bool GameProjectUtils::IsValidClassNameForCreation(const FString& NewClassName, FText& OutFailReason)
{
	if ( NewClassName.IsEmpty() )
	{
		OutFailReason = LOCTEXT( "NoClassName", "You must specify a class name." );
		return false;
	}

	if ( NewClassName.Contains(TEXT(" ")) )
	{
		OutFailReason = LOCTEXT( "ClassNameContainsSpace", "Your class name may not contain a space." );
		return false;
	}

	if ( !FChar::IsAlpha(NewClassName[0]) )
	{
		OutFailReason = LOCTEXT( "ClassNameMustBeginWithACharacter", "Your class name must begin with an alphabetic character." );
		return false;
	}

	if ( NewClassName.Len() > MAX_CLASS_NAME_LENGTH )
	{
		OutFailReason = FText::Format( LOCTEXT( "ClassNameTooLong", "The class name must not be longer than {0} characters." ), FText::AsNumber(MAX_CLASS_NAME_LENGTH) );
		return false;
	}

	FString IllegalNameCharacters;
	if ( !NameContainsOnlyLegalCharacters(NewClassName, IllegalNameCharacters) )
	{
		FFormatNamedArguments Args;
		Args.Add( TEXT("IllegalNameCharacters"), FText::FromString( IllegalNameCharacters ) );
		OutFailReason = FText::Format( LOCTEXT( "ClassNameContainsIllegalCharacters", "The class name may not contain the following characters: '{IllegalNameCharacters}'" ), Args );
		return false;
	}

	return true;
}

bool GameProjectUtils::IsValidClassNameForCreation(const FString& NewClassName, const FModuleContextInfo& ModuleInfo, const TSet<FString>& DisallowedHeaderNames, FText& OutFailReason)
{
	if (!IsValidClassNameForCreation(NewClassName, OutFailReason))
	{
		return false;
	}

	// Look for a duplicate class in memory
	for ( TObjectIterator<UClass> ClassIt; ClassIt; ++ClassIt )
	{
		if ( ClassIt->GetName() == NewClassName )
		{
			FFormatNamedArguments Args;
			Args.Add( TEXT("NewClassName"), FText::FromString( NewClassName ) );
			OutFailReason = FText::Format( LOCTEXT("ClassNameAlreadyExists", "The name {NewClassName} is already used by another class."), Args );
			return false;
		}
	}

	// Look for a duplicate class on disk in their project
	{
		FString UnusedFoundPath;
		if ( FindSourceFileInProject(NewClassName + ".h", ModuleInfo.ModuleSourcePath, UnusedFoundPath) )
		{
			FFormatNamedArguments Args;
			Args.Add( TEXT("NewClassName"), FText::FromString( NewClassName ) );
			OutFailReason = FText::Format( LOCTEXT("ClassNameAlreadyExists", "The name {NewClassName} is already used by another class."), Args );
			return false;
		}
	}

	// See if header name clashes with an engine header
	{
		FString UnusedFoundPath;
		if (DisallowedHeaderNames.Contains(NewClassName))
		{
			FFormatNamedArguments Args;
			Args.Add(TEXT("NewHeaderName"), FText::FromString(NewClassName + ".h"));
			OutFailReason = FText::Format(LOCTEXT("HeaderNameAlreadyExists", "The file {NewHeaderName} already exists elsewhere in the engine."), Args);
			return false;
		}
	}

	return true;
}

bool GameProjectUtils::IsValidBaseClassForCreation(const UClass* InClass, const FModuleContextInfo& InModuleInfo)
{
	auto DoesClassNeedAPIExport = [&InModuleInfo](const FString& InClassModuleName) -> bool
	{
		return InModuleInfo.ModuleName != InClassModuleName;
	};

	return IsValidBaseClassForCreation_Internal(InClass, FDoesClassNeedAPIExportCallback::CreateLambda(DoesClassNeedAPIExport));
}

bool GameProjectUtils::IsValidBaseClassForCreation(const UClass* InClass, const TArray<FModuleContextInfo>& InModuleInfoArray)
{
	auto DoesClassNeedAPIExport = [&InModuleInfoArray](const FString& InClassModuleName) -> bool
	{
		for(const FModuleContextInfo& ModuleInfo : InModuleInfoArray)
		{
			if(ModuleInfo.ModuleName == InClassModuleName)
			{
				return false;
			}
		}
		return true;
	};

	return IsValidBaseClassForCreation_Internal(InClass, FDoesClassNeedAPIExportCallback::CreateLambda(DoesClassNeedAPIExport));
}

bool GameProjectUtils::IsValidBaseClassForCreation_Internal(const UClass* InClass, const FDoesClassNeedAPIExportCallback& InDoesClassNeedAPIExport)
{
	// You may not make native classes based on blueprint generated classes
	const bool bIsBlueprintClass = (InClass->ClassGeneratedBy != nullptr);

	// UObject is special cased to be extensible since it would otherwise not be since it doesn't pass the API check (intrinsic class).
	const bool bIsExplicitlyUObject = (InClass == UObject::StaticClass());

	// You need API if you are not UObject itself, and you're in a module that was validated as needing API export
	const FString ClassModuleName = InClass->GetOutermost()->GetName().RightChop( FString(TEXT("/Script/")).Len() );
	const bool bNeedsAPI = !bIsExplicitlyUObject && InDoesClassNeedAPIExport.Execute(ClassModuleName);

	// You may not make a class that is not DLL exported.
	// MinimalAPI classes aren't compatible with the DLL export macro, but can still be used as a valid base
	const bool bHasAPI = InClass->HasAnyClassFlags(CLASS_RequiredAPI) || InClass->HasAnyClassFlags(CLASS_MinimalAPI);

	// @todo should we support interfaces?
	const bool bIsInterface = InClass->IsChildOf(UInterface::StaticClass());

	return !bIsBlueprintClass && (!bNeedsAPI || bHasAPI) && !bIsInterface;
}

GameProjectUtils::EAddCodeToProjectResult GameProjectUtils::AddCodeToProject(const FString& NewClassName, const FString& NewClassPath, const FModuleContextInfo& ModuleInfo, const FNewClassInfo ParentClassInfo, const TSet<FString>& DisallowedHeaderNames, FString& OutHeaderFilePath, FString& OutCppFilePath, FText& OutFailReason)
{
	EReloadStatus OutReloadStatus;
	return AddCodeToProject(NewClassName, NewClassPath, ModuleInfo, ParentClassInfo, DisallowedHeaderNames, OutHeaderFilePath, OutCppFilePath, OutFailReason, OutReloadStatus);
}

GameProjectUtils::EAddCodeToProjectResult GameProjectUtils::AddCodeToProject(const FString& NewClassName, const FString& NewClassPath, const FModuleContextInfo& ModuleInfo, const FNewClassInfo ParentClassInfo, const TSet<FString>& DisallowedHeaderNames, FString& OutHeaderFilePath, FString& OutCppFilePath, FText& OutFailReason, EReloadStatus& OutReloadStatus)
{
	const EAddCodeToProjectResult Result = AddCodeToProject_Internal(NewClassName, NewClassPath, ModuleInfo, ParentClassInfo, DisallowedHeaderNames, OutHeaderFilePath, OutCppFilePath, OutFailReason, OutReloadStatus);

	if( FEngineAnalytics::IsAvailable() )
	{
		const FString ParentClassName = ParentClassInfo.GetClassNameCPP();

		TArray<FAnalyticsEventAttribute> EventAttributes;
		EventAttributes.Add(FAnalyticsEventAttribute(TEXT("ParentClass"), ParentClassName.IsEmpty() ? TEXT("None") : ParentClassName));
		EventAttributes.Add(FAnalyticsEventAttribute(TEXT("Outcome"), Result == EAddCodeToProjectResult::Succeeded ? TEXT("Successful") : TEXT("Failed")));
		EventAttributes.Add(FAnalyticsEventAttribute(TEXT("FailureReason"), OutFailReason.ToString()));
		EventAttributes.Emplace(TEXT("Enterprise"), IProjectManager::Get().IsEnterpriseProject());

		FEngineAnalytics::GetProvider().RecordEvent( TEXT( "Editor.AddCodeToProject.CodeAdded" ), EventAttributes );
	}

	return Result;
}

UTemplateCategories* GameProjectUtils::LoadTemplateCategories(const FString& RootDir)
{
	UTemplateCategories* TemplateCategories = nullptr;

	FString TemplateCategoriesIniFilename = RootDir / TEXT("TemplateCategories.ini");
	if (FPlatformFileManager::Get().GetPlatformFile().FileExists(*TemplateCategoriesIniFilename))
	{
		TemplateCategories = NewObject<UTemplateCategories>();
		TemplateCategories->LoadConfig(UTemplateCategories::StaticClass(), *TemplateCategoriesIniFilename);

		for (FTemplateCategoryDef& Category : TemplateCategories->Categories)
		{
			// attempt to resolve the icon relative to the root directory
			FString TemplateCategoryIcon = RootDir / Category.Icon;
			if (FPlatformFileManager::Get().GetPlatformFile().FileExists(*TemplateCategoryIcon))
			{
				Category.Icon = TemplateCategoryIcon;
			}
		}
	}

	return TemplateCategories;
}

UTemplateProjectDefs* GameProjectUtils::LoadTemplateDefs(const FString& ProjectDirectory)
{
	UTemplateProjectDefs* TemplateDefs = nullptr;

	const FString TemplateDefsIniFilename = FConfigCacheIni::NormalizeConfigIniPath(ProjectDirectory / TEXT("Config") / GetTemplateDefsFilename());
	if ( FPlatformFileManager::Get().GetPlatformFile().FileExists(*TemplateDefsIniFilename) )
	{
		UClass* ClassToConstruct = UDefaultTemplateProjectDefs::StaticClass();

		// see if template uses a custom project defs object
		FString ClassName;
		const bool bFoundValue = GConfig->GetString(*UTemplateProjectDefs::StaticClass()->GetPathName(), TEXT("TemplateProjectDefsClass"), ClassName, TemplateDefsIniFilename);
		if (bFoundValue && ClassName.Len() > 0)
		{
			UClass* OverrideClass = UClass::TryFindTypeSlow<UClass>(ClassName);
			if (nullptr != OverrideClass)
			{
				ClassToConstruct = OverrideClass;
			}
			else
			{
				UE_LOGF(LogGameProjectGeneration, Error, "Failed to find template project defs class '%ls', using default.", *ClassName);
			}
		}

		TemplateDefs = NewObject<UTemplateProjectDefs>(GetTransientPackage(), ClassToConstruct);
		TemplateDefs->LoadConfig(UTemplateProjectDefs::StaticClass(), *TemplateDefsIniFilename);

		if (TemplateDefs->HiddenSettings.Num() > 1 && TemplateDefs->HiddenSettings.Contains(ETemplateSetting::All))
		{
			UE_LOGF(LogGameProjectGeneration, Warning, "Template '%ls' contains 'All' in HiddenSettings in addition to other entries. This is a mistake, and means that all settings will be hidden.", *ProjectDirectory);
		}
		
		if (TemplateDefs->Variants.Num() > 0) // We make sure the Variants are valid
		{
			TSet<FName> VariantNames;
			for (TArray<FTemplateVariant>::TIterator It = TemplateDefs->Variants.CreateIterator(); It; ++It)
			{
				const FTemplateVariant& Variant = *It;
				if (VariantNames.Contains(Variant.Name))
				{
					UE_LOGF(LogGameProjectGeneration, Warning, "Template '%ls' contains multiple Variants with the name '%ls', only the first one will be kept.", *ProjectDirectory, *Variant.Name.ToString());
				}
				else if (Variant.Name.IsNone()) // NAME_None is used to represent no Variant selected
				{
					UE_LOGF(LogGameProjectGeneration, Error, "Template '%ls' contains a Variant with the name '%ls' which is not allowed, it will be discarded.", *ProjectDirectory, *Variant.Name.ToString());
				}
				else
				{
					VariantNames.Add(Variant.Name);
					continue;
				}
				It.RemoveCurrent();
			}
		}
	}

	return TemplateDefs;
}

TOptional<FGuid> GameProjectUtils::GenerateProjectFromScratch(const FProjectInformation& InProjectInfo, FText& OutFailReason, FText& OutFailLog)
{
	FScopedSlowTask SlowTask(5);

	const FString NewProjectFolder = FPaths::GetPath(InProjectInfo.ProjectFilename);
	const FString NewProjectName = FPaths::GetBaseFilename(InProjectInfo.ProjectFilename);
	TArray<FString> CreatedFiles;

	SlowTask.EnterProgressFrame();

	ResetCurrentProjectModulesCache();

	FGuid ProjectID;
	// Generate config files
	if (!GenerateConfigFiles(InProjectInfo, CreatedFiles, OutFailReason, ProjectID))
	{
		return TOptional<FGuid>();
	}

	// Make the Content folder
	const FString ContentFolder = NewProjectFolder / TEXT("Content");
	if ( !IFileManager::Get().MakeDirectory(*ContentFolder) )
	{
		FFormatNamedArguments Args;
		Args.Add( TEXT("ContentFolder"), FText::FromString( ContentFolder ) );
		OutFailReason = FText::Format( LOCTEXT("FailedToCreateContentFolder", "Failed to create the content folder {ContentFolder}"), Args );
		return TOptional<FGuid>();
	}

	SlowTask.EnterProgressFrame();

	TArray<FString> StartupModuleNames;
	if ( InProjectInfo.bShouldGenerateCode )
	{
		FScopedSlowTask LocalScope(2);

		LocalScope.EnterProgressFrame();
		// Generate basic source code files
		if ( !GenerateBasicSourceCode(NewProjectFolder / TEXT("Source"), NewProjectName, NewProjectFolder, StartupModuleNames, CreatedFiles, OutFailReason) )
		{
			return TOptional<FGuid>();
		}

		LocalScope.EnterProgressFrame();
		// Generate game framework source code files
		if ( !GenerateGameFrameworkSourceCode(NewProjectFolder / TEXT("Source"), NewProjectName, CreatedFiles, OutFailReason) )
		{
			return TOptional<FGuid>();
		}
	}

	SlowTask.EnterProgressFrame();

	// Generate the project file
	{
		// Set up the descriptor
		FProjectDescriptor Project;
		for(int32 Idx = 0; Idx < StartupModuleNames.Num(); Idx++)
		{
			Project.Modules.Add(FModuleDescriptor(*StartupModuleNames[Idx]));
		}
		
		//=====================================================================
		// Explicitly enable Modeling Mode plugin in Blank Template,
		// with AllowList=Editor flags. In 5.0 the Modeling Mode plugin
		// cannot be enabledByDefault in the .uplugin file due to 
		// dependent Runtime modules that should not be included in all
		// game builds. So, In 5.0 the plugin is explicitly enabled here
		// for Blank projects. The uplugin-level issue is expected to 
		// be resolved in 5.1, at which point this code block will be deleted
		//=====================================================================
		TSharedPtr<IPlugin> ModelingModePlugin = IPluginManager::Get().FindPlugin(TEXT("ModelingToolsEditorMode"));
		if ( ModelingModePlugin.IsValid() )
		{
			FPluginReferenceDescriptor ModelingModeDescriptor(ModelingModePlugin->GetName(), true);
			ModelingModeDescriptor.TargetAllowList.Add(EBuildTargetType::Editor);
			Project.Plugins.Add(ModelingModeDescriptor);
		}

		Project.bIsEnterpriseProject = InProjectInfo.bIsEnterpriseProject;

		// Try to save it
		FText LocalFailReason;
		if(!Project.Save(InProjectInfo.ProjectFilename, LocalFailReason))
		{
			OutFailReason = LocalFailReason;
			return TOptional<FGuid>();
		}
		CreatedFiles.Add(InProjectInfo.ProjectFilename);

		// Set the engine identifier for it. Do this after saving, so it can be correctly detected as foreign or non-foreign.
		if(!SetEngineAssociationForForeignProject(InProjectInfo.ProjectFilename, OutFailReason))
		{
			return TOptional<FGuid>();
		}
	}

	SlowTask.EnterProgressFrame();

	if ( InProjectInfo.bShouldGenerateCode )
	{
		// Generate project files
		if ( !GenerateCodeProjectFiles(InProjectInfo.ProjectFilename, OutFailReason, OutFailLog) )
		{
			return TOptional<FGuid>();
		}
	}

	SlowTask.EnterProgressFrame();

	UE_LOGF(LogGameProjectGeneration, Log, "Created new project with %d files (plus project files)", CreatedFiles.Num());
	return ProjectID;
}

static bool SaveConfigValues(const FProjectInformation& InProjectInfo, const TArray<FTemplateConfigValue>& ConfigValues, FText& OutFailReason)
{
	const FString ProjectConfigPath = FPaths::GetPath(InProjectInfo.ProjectFilename) / TEXT("Config");

	// Fix all specified config values
	for (const FTemplateConfigValue& ConfigValue : ConfigValues)
	{
		const FString IniFilename = ProjectConfigPath / ConfigValue.ConfigFile;
		bool bSuccessfullyProcessed = false;

		TArray<FString> FileLines;
		if (FFileHelper::LoadANSITextFileToStrings(*IniFilename, &IFileManager::Get(), FileLines))
		{
			FString FileOutput;
			const FString TargetSection = ConfigValue.ConfigSection;
			FString CurSection;
			bool bFoundTargetKey = false;
			for (const FString& LineIn : FileLines)
			{
				FString Line = LineIn;
				Line.TrimStartAndEndInline();

				bool bShouldExcludeLineFromOutput = false;

				// If we not yet found the target key parse each line looking for it
				if (!bFoundTargetKey)
				{
					if (Line.Len() == 0)
					{
						// Check for an empty line. No work needs to be done on these lines
					}
					else if (Line.StartsWith(TEXT(";")))
					{
						// Comment lines start with ";". Skip these lines entirely.
					}
					else if (Line.StartsWith(TEXT("[")))
					{
						// If this is a section line, update the section
						// If we are entering a new section and we have not yet found our key in the target section, add it to the end of the section
						if (CurSection == TargetSection)
						{
							FileOutput += ConfigValue.ConfigKey + TEXT("=") + ConfigValue.ConfigValue + LINE_TERMINATOR + LINE_TERMINATOR;
							bFoundTargetKey = true;
						}

						// Update the current section
						CurSection = Line.Mid(1, Line.Len() - 2);
					}
					// This is possibly an actual key/value pair
					else if (CurSection == TargetSection)
					{
						// Key value pairs contain an equals sign
						const int32 EqualsIdx = Line.Find(TEXT("="));
						if (EqualsIdx != INDEX_NONE)
						{
							// Determine the key and see if it is the target key
							const FString Key = Line.Left(EqualsIdx);
							if (Key == ConfigValue.ConfigKey)
							{
								// Found the target key, add it to the output and skip the current line if the target value is supposed to replace
								FileOutput += ConfigValue.ConfigKey + TEXT("=") + ConfigValue.ConfigValue + LINE_TERMINATOR;
								bShouldExcludeLineFromOutput = ConfigValue.bShouldReplaceExistingValue;
								bFoundTargetKey = true;
							}
						}
					}
				}

				// Unless we replaced the key, add this line to the output
				if (!bShouldExcludeLineFromOutput)
				{
					FileOutput += Line;
					if (&LineIn != &FileLines.Last())
					{
						// Add a line terminator on every line except the last
						FileOutput += LINE_TERMINATOR;
					}
				}
			}

			// If the key did not exist, add it here
			if (!bFoundTargetKey)
			{
				// If we did not end in the correct section, add the section to the bottom of the file
				if (CurSection != TargetSection)
				{
					FileOutput += LINE_TERMINATOR;
					FileOutput += FString::Printf(TEXT("[%s]"), *TargetSection) + LINE_TERMINATOR;
				}

				// Add the key/value here
				FileOutput += ConfigValue.ConfigKey + TEXT("=") + ConfigValue.ConfigValue + LINE_TERMINATOR;
			}

			if (!FFileHelper::SaveStringToFile(FileOutput, *IniFilename))
			{
				FFormatNamedArguments Args;
				Args.Add(TEXT("ConfigFile"), FText::FromString(IniFilename));
				OutFailReason = FText::Format(LOCTEXT("FailedToFixUpConfigFile", "Failed to process file {ConfigFile}."), Args);
				return false;
			}
		}
	}

	return true;
}

static FString GetReplacePlaceholder(int Idx)
{
	return FString::Printf(TEXT("{{{REPLACE:%d}}}"), Idx);
}

TOptional<FGuid> GameProjectUtils::CreateProjectFromTemplate(const FProjectInformation& InProjectInfo, FText& OutFailReason, FText& OutFailLog, TArray<FString>* OutCreatedFiles)
{
	FScopedSlowTask SlowTask(10);

	const FString ProjectName = FPaths::GetBaseFilename(InProjectInfo.ProjectFilename);
	const FString TemplateName = FPaths::GetBaseFilename(InProjectInfo.TemplateFile);
	const FString SrcFolder = FPaths::GetPath(InProjectInfo.TemplateFile);
	const FString DestFolder = FPaths::GetPath(InProjectInfo.ProjectFilename);

	if (!FPlatformFileManager::Get().GetPlatformFile().FileExists(*InProjectInfo.TemplateFile))
	{
		FFormatNamedArguments Args;
		Args.Add( TEXT("TemplateFile"), FText::FromString( InProjectInfo.TemplateFile ) );
		OutFailReason = FText::Format( LOCTEXT("InvalidTemplate_MissingProject", "Template project \"{TemplateFile}\" does not exist."), Args );
		return TOptional<FGuid>();
	}

	SlowTask.EnterProgressFrame();

	UTemplateProjectDefs* TemplateDefs = LoadTemplateDefs(SrcFolder);
	if (TemplateDefs == nullptr)
	{
		FFormatNamedArguments Args;
		Args.Add( TEXT("TemplateFile"), FText::FromString( FPaths::GetBaseFilename(InProjectInfo.TemplateFile) ) );
		Args.Add( TEXT("TemplateDefinesFile"), FText::FromString( GetTemplateDefsFilename() ) );
		OutFailReason = FText::Format( LOCTEXT("InvalidTemplate_MissingDefs", "Template project \"{TemplateFile}\" does not have definitions file: '{TemplateDefinesFile}'."), Args );
		return TOptional<FGuid>();
	}

	SlowTask.EnterProgressFrame();

	// create a Content folder in the destination directory
	const FString DestContentFolder = DestFolder / TEXT("Content");
	if (!IFileManager::Get().MakeDirectory(*DestContentFolder, true))
	{
		FFormatNamedArguments FailArgs;
		FailArgs.Add(TEXT("DestContentFolder"), FText::FromString(DestContentFolder));
		OutFailReason = FText::Format(LOCTEXT("FailedToCreateDirectory", "Failed to create directory \"{DestContentFolder}\"."), FailArgs);
		return TOptional<FGuid>();
	}

	// Add a rule to replace the build settings version with the appropriate number
	FTemplateReplacement& DefaultBuildSettingsRepl = TemplateDefs->ReplacementsInFiles.AddZeroed_GetRef();
	DefaultBuildSettingsRepl.Extensions.Add("cs");
	DefaultBuildSettingsRepl.From = TEXT("BuildSettingsVersion.Latest");
	DefaultBuildSettingsRepl.To = GetDefaultBuildSettingsVersion();
	DefaultBuildSettingsRepl.bCaseSensitive = true;

	// Fix up the replacement strings using the specified project name
	TemplateDefs->FixupStrings(TemplateName, ProjectName);

	// Form a list of all extensions we care about
	TSet<FString> ReplacementsInFilesExtensions;
	for ( const FTemplateReplacement& Replacement : TemplateDefs->ReplacementsInFiles )
	{
		ReplacementsInFilesExtensions.Append(Replacement.Extensions);
	}

	// Keep a list of created files so we can delete them if project creation fails
	TArray<FString> CreatedFiles;

	SlowTask.EnterProgressFrame();

	// Discover and copy all files in the src folder to the destination, excluding a few files and folders
	TArray<FString> FilesToCopy;
	TArray<FString> FilesThatNeedContentsReplaced;
	TMap<FString, FString> ClassRenames;
	IFileManager::Get().FindFilesRecursive(FilesToCopy, *SrcFolder, TEXT("*"), /*Files=*/true, /*Directories=*/false);

	SlowTask.EnterProgressFrame();
	{
		// Open a new feedback scope for the loop so we can report how far through the copy we are
		FScopedSlowTask InnerSlowTask(static_cast<float>(FilesToCopy.Num()));
		for ( const FString& SrcFilename : FilesToCopy )
		{
			// Update the progress
			FFormatNamedArguments Args;
			Args.Add( TEXT("SrcFilename"), FText::FromString( FPaths::GetCleanFilename(SrcFilename) ) );
			InnerSlowTask.EnterProgressFrame(1, FText::Format( LOCTEXT( "CreatingProjectStatus_CopyingFile", "Copying File {SrcFilename}..." ), Args ));

			// Get the file path, relative to the src folder
			const FString SrcFileSubpath = SrcFilename.RightChop(SrcFolder.Len() + 1);

			// Skip any files that were configured to be ignored
			if ( TemplateDefs->FilesToIgnore.Contains(SrcFileSubpath) )
			{
				// This file was marked as "ignored"
				continue;
			}

			// Skip any folders that were configured to be ignored
			if ( const FString* IgnoredFolder = TemplateDefs->FoldersToIgnore.FindByPredicate([&SrcFileSubpath](const FString& Ignore){ return SrcFileSubpath.StartsWith(Ignore + TEXT("/")); }) )
			{
				// This folder was marked as "ignored"
				UE_LOGF(LogGameProjectGeneration, Verbose, "'%ls': Skipping as it is in an ignored folder '%ls'", *SrcFilename, **IgnoredFolder);
				continue;
			}

			// Retarget any folders that were chosen to be renamed by choosing a new destination subpath now
			FString DestFileSubpathWithoutFilename = FPaths::GetPath(SrcFileSubpath) + TEXT("/");
			for ( const FTemplateFolderRename& FolderRename : TemplateDefs->FolderRenames )
			{
				if ( SrcFileSubpath.StartsWith(FolderRename.From + TEXT("/")) )
				{
					// This was a file in a renamed folder. Retarget to the new location
					DestFileSubpathWithoutFilename = FolderRename.To / DestFileSubpathWithoutFilename.RightChop( FolderRename.From.Len() );
					UE_LOGF(LogGameProjectGeneration, Verbose, "'%ls': Moving to '%ls' as it matched folder rename ('%ls'->'%ls')", *SrcFilename, *DestFileSubpathWithoutFilename, *FolderRename.From, *FolderRename.To);
				}
			}

			// Retarget any files that were chosen to have parts of their names replaced here
			FString DestBaseFilename = FPaths::GetBaseFilename(SrcFileSubpath);
			const FString FileExtension = FPaths::GetExtension(SrcFileSubpath);
			for ( const FTemplateReplacement& Replacement : TemplateDefs->FilenameReplacements )
			{
				if ( Replacement.Extensions.Contains( FileExtension ) )
				{
					// This file matched a filename replacement extension, apply it now
					FString LastDestBaseFilename = DestBaseFilename;
					DestBaseFilename = DestBaseFilename.Replace(*Replacement.From, *Replacement.To, Replacement.bCaseSensitive ? ESearchCase::CaseSensitive : ESearchCase::IgnoreCase);

					if (LastDestBaseFilename != DestBaseFilename)
					{
						UE_LOGF(LogGameProjectGeneration, Verbose, "'%ls': Renaming to '%ls/%ls' as it matched file rename ('%ls'->'%ls')", *SrcFilename, *DestFileSubpathWithoutFilename, *DestBaseFilename, *Replacement.From, *Replacement.To);
					}
				}
			}

			// Perform the copy
			const FString DestFilename = DestFolder / DestFileSubpathWithoutFilename + DestBaseFilename + TEXT(".") + FileExtension;
			if ( IFileManager::Get().Copy(*DestFilename, *SrcFilename) == COPY_OK )
			{
				CreatedFiles.Add(DestFilename);

				if ( ReplacementsInFilesExtensions.Contains(FileExtension) )
				{
					FilesThatNeedContentsReplaced.Add(DestFilename);
				}

				// Allow project template to extract class renames from this file copy
				if (FPaths::GetBaseFilename(SrcFilename) != FPaths::GetBaseFilename(DestFilename)
					&& TemplateDefs->IsClassRename(DestFilename, SrcFilename, FileExtension))
				{
					// Looks like a UObject file!
					ClassRenames.Add(FPaths::GetBaseFilename(SrcFilename), FPaths::GetBaseFilename(DestFilename));
				}
			}
			else
			{
				FFormatNamedArguments FailArgs;
				FailArgs.Add(TEXT("SrcFilename"), FText::FromString(SrcFilename));
				FailArgs.Add(TEXT("DestFilename"), FText::FromString(DestFilename));
				OutFailReason = FText::Format(LOCTEXT("FailedToCopyFile", "Failed to copy \"{SrcFilename}\" to \"{DestFilename}\"."), FailArgs);
				return TOptional<FGuid>();
			}
		}
	}

	SlowTask.EnterProgressFrame();
	{
		// Open a new feedback scope for the loop so we can report how far through the process we are
		FScopedSlowTask InnerSlowTask(static_cast<float>(FilesThatNeedContentsReplaced.Num()));

		// Open all files with the specified extensions and replace text
		for ( const FString& FileToFix : FilesThatNeedContentsReplaced )
		{
			InnerSlowTask.EnterProgressFrame();

			bool bSuccessfullyProcessed = false;

			FString FileContents;
			if ( FFileHelper::LoadFileToString(FileContents, *FileToFix) )
			{
				// Substitute strings in two passes to avoid situations where patterns may match the replaced strings.
				for (int Idx = 0; Idx < TemplateDefs->ReplacementsInFiles.Num(); Idx++)
				{
					const FTemplateReplacement& Replacement = TemplateDefs->ReplacementsInFiles[Idx];
					if (Replacement.Extensions.Contains(FPaths::GetExtension(FileToFix)))
					{
						FString Placeholder = GetReplacePlaceholder(Idx);
						FileContents.ReplaceInline(*Replacement.From, *Placeholder, Replacement.bCaseSensitive ? ESearchCase::CaseSensitive : ESearchCase::IgnoreCase);
					}
				}
				for (int Idx = 0; Idx < TemplateDefs->ReplacementsInFiles.Num(); Idx++)
				{
					const FTemplateReplacement& Replacement = TemplateDefs->ReplacementsInFiles[Idx];
					if (Replacement.Extensions.Contains(FPaths::GetExtension(FileToFix)))
					{
						FString Placeholder = GetReplacePlaceholder(Idx);
						FileContents.ReplaceInline(*Placeholder, *Replacement.To, ESearchCase::CaseSensitive);
					}
				}

				if ( FFileHelper::SaveStringToFile(FileContents, *FileToFix) )
				{
					bSuccessfullyProcessed = true;
				}
			}

			if ( !bSuccessfullyProcessed )
			{
				FFormatNamedArguments Args;
				Args.Add( TEXT("FileToFix"), FText::FromString( FileToFix ) );
				OutFailReason = FText::Format( LOCTEXT("FailedToFixUpFile", "Failed to process file \"{FileToFix}\"."), Args );
				return TOptional<FGuid>();
			}
		}
	}

	SlowTask.EnterProgressFrame();

	// Fixup specific ini values
	TArray<FTemplateConfigValue> ConfigValuesToSet;

	AddHardwareConfigValues(InProjectInfo, ConfigValuesToSet);

	AddLumenConfigValues(InProjectInfo, ConfigValuesToSet, /* do not disable static lighting, as some templates need it */ false);
	AddRaytracingConfigValues(InProjectInfo, ConfigValuesToSet);
	AddSubstrateConfigValues(InProjectInfo, ConfigValuesToSet);
	AddNewProjectDefaultShadowConfigValues(InProjectInfo, ConfigValuesToSet);
	AddPostProcessingConfigValues(InProjectInfo, ConfigValuesToSet);
	AddWorldPartitionConfigValues(InProjectInfo, ConfigValuesToSet);
	AddUserInterfaceEngineConfigValues(InProjectInfo, ConfigValuesToSet);
	AddUserInterfaceGameConfigValues(InProjectInfo, ConfigValuesToSet);
	
	TemplateDefs->AddConfigValues(ConfigValuesToSet, TemplateName, ProjectName, InProjectInfo.bShouldGenerateCode);

	FGuid ProjectID = FGuid::NewGuid();
	ConfigValuesToSet.Emplace(TEXT("DefaultGame.ini"), TEXT("/Script/EngineSettings.GeneralProjectSettings"), TEXT("ProjectID"), ProjectID.ToString(), /*InShouldReplaceExistingValue=*/true);

	// Add all classname fixups
	for (const TPair<FString, FString>& Rename : ClassRenames)
	{
		const FString ClassRedirectString = FString::Printf(TEXT("(OldClassName=\"%s\",NewClassName=\"%s\")"), *Rename.Key, *Rename.Value);
		ConfigValuesToSet.Emplace(TEXT("DefaultEngine.ini"), TEXT("/Script/Engine.Engine"), TEXT("+ActiveClassRedirects"), *ClassRedirectString, /*InShouldReplaceExistingValue=*/false);
	}

	SlowTask.EnterProgressFrame();

	if (!SaveConfigValues(InProjectInfo, ConfigValuesToSet, OutFailReason))
	{
		return TOptional<FGuid>();
	}

	if (!AddSharedContentToProject(InProjectInfo, CreatedFiles, OutFailReason))
	{
		return TOptional<FGuid>();
	}

	if (!TemplateDefs->PreGenerateProject(DestFolder, SrcFolder, InProjectInfo.ProjectFilename, InProjectInfo.TemplateFile, InProjectInfo.bShouldGenerateCode, OutFailReason))
	{
		return TOptional<FGuid>();
	}

	SlowTask.EnterProgressFrame();

	// Generate the project file
	{
		// Load the source project
		FProjectDescriptor Project;
		if (!Project.Load(InProjectInfo.TemplateFile, OutFailReason))
		{
			return TOptional<FGuid>();
		}

		// Update it to current
		Project.EngineAssociation.Empty();
		Project.EpicSampleNameHash = 0;

		// Fix up module names
		const FString BaseSourceName = FPaths::GetBaseFilename(InProjectInfo.TemplateFile);
		const FString BaseNewName = FPaths::GetBaseFilename(InProjectInfo.ProjectFilename);
		for (FModuleDescriptor& ModuleInfo : Project.Modules)
		{
			ModuleInfo.Name = FName(*ModuleInfo.Name.ToString().Replace(*BaseSourceName, *BaseNewName));
		}

		// Save it to disk
		if (!Project.Save(InProjectInfo.ProjectFilename, OutFailReason))
		{
			return TOptional<FGuid>();
		}

		// Set the engine identifier if it's a foreign project. Do this after saving, so it can be correctly detected as foreign.
		if (!SetEngineAssociationForForeignProject(InProjectInfo.ProjectFilename, OutFailReason))
		{
			return TOptional<FGuid>();
		}

		// Add it to the list of created files
		CreatedFiles.Add(InProjectInfo.ProjectFilename);
	}

	SlowTask.EnterProgressFrame();

	if (InProjectInfo.bShouldGenerateCode)
	{
		// Generate project files
		if (!GenerateCodeProjectFiles(InProjectInfo.ProjectFilename, OutFailReason, OutFailLog))
		{
			return TOptional<FGuid>();
		}
	}

	SlowTask.EnterProgressFrame();

	if (!TemplateDefs->PostGenerateProject(DestFolder, SrcFolder, InProjectInfo.ProjectFilename, InProjectInfo.TemplateFile, InProjectInfo.bShouldGenerateCode, OutFailReason))
	{
		return TOptional<FGuid>();
	}

	if (OutCreatedFiles != nullptr)
	{
		OutCreatedFiles->Append(CreatedFiles);
	}

	return ProjectID;
}

bool GameProjectUtils::SetEngineAssociationForForeignProject(const FString& ProjectFileName, FText& OutFailReason)
{
	if(FUProjectDictionary(FPaths::RootDir()).IsForeignProject(ProjectFileName))
	{
		if(!FDesktopPlatformModule::Get()->SetEngineIdentifierForProject(ProjectFileName, FDesktopPlatformModule::Get()->GetCurrentEngineIdentifier()))
		{
			OutFailReason = LOCTEXT("FailedToSetEngineIdentifier", "Couldn't set engine identifier for project");
			return false;
		}
	}
	return true;
}

FString GameProjectUtils::GetTemplateDefsFilename()
{
	return TEXT("TemplateDefs.ini");
}

FString GameProjectUtils::GetIncludePathForFile(const FString& InFullFilePath, const FString& ModuleRootPath)
{
	FString RelativeHeaderPath = FPaths::ChangeExtension(InFullFilePath, "h");

	const FString PublicString = ModuleRootPath / "Public" / "";
	if (RelativeHeaderPath.StartsWith(PublicString))
	{
		return RelativeHeaderPath.RightChop(PublicString.Len());
	}

	const FString PrivateString = ModuleRootPath / "Private" / "";
	if (RelativeHeaderPath.StartsWith(PrivateString))
	{
		return RelativeHeaderPath.RightChop(PrivateString.Len());
	}
	
	return RelativeHeaderPath.RightChop(ModuleRootPath.Len());
}

bool GameProjectUtils::NameContainsOnlyLegalCharacters(const FString& TestName, FString& OutIllegalCharacters)
{
	bool bContainsIllegalCharacters = false;

	// Only allow alphanumeric characters in the project name
	bool bFoundAlphaNumericChar = false;
	for ( int32 CharIdx = 0 ; CharIdx < TestName.Len() ; ++CharIdx )
	{
		const FString& Char = TestName.Mid( CharIdx, 1 );
		if ( !FChar::IsAlnum(Char[0]) && Char != TEXT("_") )
		{
			if ( !OutIllegalCharacters.Contains( Char ) )
			{
				OutIllegalCharacters += Char;
			}

			bContainsIllegalCharacters = true;
		}
	}

	return !bContainsIllegalCharacters;
}

bool GameProjectUtils::NameMatchesPlatformModuleName(const FString& TestName)
{
	for (auto Pair : FDataDrivenPlatformInfoRegistry::GetAllPlatformInfos())
	{
		FString PlatformNameString = Pair.Key.ToString();
		if ((PlatformNameString == TestName) || ((PlatformNameString += "TargetPlatform") == TestName))
		{
			return true;
		}
	}
	TArray<FString> CustomTargetPlatformModules;
	GConfig->GetArray(TEXT("CustomTargetPlatforms"), TEXT("ModuleName"), CustomTargetPlatformModules, GEditorIni);
	for (const FString& ModuleName : CustomTargetPlatformModules)
	{
		if (TestName == ModuleName)
		{
			return true;
		}
	}
	return false;
}

bool GameProjectUtils::ProjectFileExists(const FString& ProjectFile)
{
	return FPlatformFileManager::Get().GetPlatformFile().FileExists(*ProjectFile);
}

bool GameProjectUtils::AnyProjectFilesExistInFolder(const FString& Path)
{
	TArray<FString> ExistingFiles;
	const FString Wildcard = FString::Printf(TEXT("%s/*.%s"), *Path, *FProjectDescriptor::GetExtension());
	IFileManager::Get().FindFiles(ExistingFiles, *Wildcard, /*Files=*/true, /*Directories=*/false);

	return ExistingFiles.Num() > 0;
}

bool GameProjectUtils::CleanupIsEnabled()
{
	// Clean up files when running Rocket (unless otherwise specified on the command line)
	return FParse::Param(FCommandLine::Get(), TEXT("norocketcleanup")) == false;
}

void GameProjectUtils::DeleteCreatedFiles(const FString& RootFolder, const TArray<FString>& CreatedFiles)
{
	if (CleanupIsEnabled())
	{
		for ( auto FileToDeleteIt = CreatedFiles.CreateConstIterator(); FileToDeleteIt; ++FileToDeleteIt )
		{
			IFileManager::Get().Delete(**FileToDeleteIt);
		}

		// If the project folder is empty after deleting all the files we created, delete the directory as well
		TArray<FString> RemainingFiles;
		IFileManager::Get().FindFilesRecursive(RemainingFiles, *RootFolder, TEXT("*.*"), /*Files=*/true, /*Directories=*/false);
		if ( RemainingFiles.Num() == 0 )
		{
			IFileManager::Get().DeleteDirectory(*RootFolder, /*RequireExists=*/false, /*Tree=*/true);
		}
	}
}

void GameProjectUtils::AddHardwareConfigValues(const FProjectInformation& InProjectInfo, TArray<FTemplateConfigValue>& ConfigValues)
{
	if (InProjectInfo.TargetedHardware.IsSet())
	{
		UEnum* HardwareClassEnum = StaticEnum<EHardwareClass>();
		if (HardwareClassEnum != nullptr)
		{
			FString TargetHardwareString;
			HardwareClassEnum->GetValueAsString(InProjectInfo.TargetedHardware.GetValue(), /*out*/ TargetHardwareString);

			if (!TargetHardwareString.IsEmpty())
			{
				ConfigValues.Emplace(TEXT("DefaultEngine.ini"),
					TEXT("/Script/HardwareTargeting.HardwareTargetingSettings"),
					TEXT("TargetedHardwareClass"),
					TargetHardwareString,
					true /* ShouldReplaceExistingValue */);
			}
		}
	}

	if (InProjectInfo.DefaultGraphicsPerformance.IsSet())
	{
		UEnum* GraphicsPresetEnum = StaticEnum<EGraphicsPreset>();
		if (GraphicsPresetEnum != nullptr)
		{
			FString GraphicsPresetString;
			GraphicsPresetEnum->GetValueAsString(InProjectInfo.DefaultGraphicsPerformance.GetValue(), /*out*/ GraphicsPresetString);

			if (!GraphicsPresetString.IsEmpty())
			{
				ConfigValues.Emplace(TEXT("DefaultEngine.ini"),
					TEXT("/Script/HardwareTargeting.HardwareTargetingSettings"),
					TEXT("DefaultGraphicsPerformance"),
					GraphicsPresetString,
					true /* ShouldReplaceExistingValue */);
			}
		}
	}

	// Don't override these settings for templates
	if (InProjectInfo.TemplateFile.IsEmpty())
	{
		// New projects always have DX12 by default on Windows
		ConfigValues.Emplace(TEXT("DefaultEngine.ini"),
			TEXT("/Script/WindowsTargetPlatform.WindowsTargetSettings"),
			TEXT("DefaultGraphicsRHI"),
			TEXT("DefaultGraphicsRHI_DX12"),
			false /* ShouldReplaceExistingValue */);

		// Force clear D3D12TargetedShaderFormats since the BaseEngine list can change at any time.
		ConfigValues.Emplace(TEXT("DefaultEngine.ini"),
			TEXT("/Script/WindowsTargetPlatform.WindowsTargetSettings"),
			TEXT("!D3D12TargetedShaderFormats"),
			TEXT("ClearArray"),
			false /* ShouldReplaceExistingValue */);

		// New projects always have DX12 only supporting SM6 by default on Windows
		ConfigValues.Emplace(TEXT("DefaultEngine.ini"),
			TEXT("/Script/WindowsTargetPlatform.WindowsTargetSettings"),
			TEXT("+D3D12TargetedShaderFormats"),
			TEXT("PCD3D_SM6"),
			false /* ShouldReplaceExistingValue */);
	}
}

bool GameProjectUtils::GenerateConfigFiles(const FProjectInformation& InProjectInfo, TArray<FString>& OutCreatedFiles, FText& OutFailReason, FGuid& OutProjectID)
{
	const FString NewProjectFolder = FPaths::GetPath(InProjectInfo.ProjectFilename);
	const FString NewProjectName = FPaths::GetBaseFilename(InProjectInfo.ProjectFilename);

	FString ProjectConfigPath = NewProjectFolder / TEXT("Config");

	// DefaultEngine.ini
	{
		const FString DefaultEngineIniFilename = ProjectConfigPath / TEXT("DefaultEngine.ini");
		FString FileContents;

		FileContents += LINE_TERMINATOR;
		FileContents += TEXT("[Audio]") LINE_TERMINATOR;
		FileContents += TEXT("UseAudioMixer=True") LINE_TERMINATOR;

		if (WriteOutputFile(DefaultEngineIniFilename, FileContents, OutFailReason))
		{
			OutCreatedFiles.Add(DefaultEngineIniFilename);
		}
		else
		{
			return false;
		}

		TArray<FTemplateConfigValue> ConfigValuesToSet;
		AddHardwareConfigValues(InProjectInfo, ConfigValuesToSet);
		AddLumenConfigValues(InProjectInfo, ConfigValuesToSet);
		AddNewProjectDefaultShadowConfigValues(InProjectInfo, ConfigValuesToSet);
		AddPostProcessingConfigValues(InProjectInfo, ConfigValuesToSet);
		AddRaytracingConfigValues(InProjectInfo, ConfigValuesToSet);
		AddSubstrateConfigValues(InProjectInfo, ConfigValuesToSet);
		AddWorldPartitionConfigValues(InProjectInfo, ConfigValuesToSet);
		AddUserInterfaceEngineConfigValues(InProjectInfo, ConfigValuesToSet);

		if (!SaveConfigValues(InProjectInfo, ConfigValuesToSet, OutFailReason))
		{
			return false;
		}
	}

	// DefaultEditor.ini
	{
		const FString DefaultEditorIniFilename = ProjectConfigPath / TEXT("DefaultEditor.ini");
		FString FileContents;

		if (WriteOutputFile(DefaultEditorIniFilename, FileContents, OutFailReason))
		{
			OutCreatedFiles.Add(DefaultEditorIniFilename);
		}
		else
		{
			return false;
		}
	}

	// DefaultGame.ini
	{
		const FString DefaultGameIniFilename = ProjectConfigPath / TEXT("DefaultGame.ini");
		FString FileContents;
		FileContents += TEXT("[/Script/EngineSettings.GeneralProjectSettings]") LINE_TERMINATOR;

		OutProjectID = FGuid::NewGuid();
		FileContents += TEXT("ProjectID=") + OutProjectID.ToString() + LINE_TERMINATOR;

		if (WriteOutputFile(DefaultGameIniFilename, FileContents, OutFailReason))
		{
			OutCreatedFiles.Add(DefaultGameIniFilename);
		}
		else
		{
			return false;
		}

		TArray<FTemplateConfigValue> ConfigValuesToSet;
		AddUserInterfaceGameConfigValues(InProjectInfo, ConfigValuesToSet);

		if (!SaveConfigValues(InProjectInfo, ConfigValuesToSet, OutFailReason))
		{
			return false;
		}
	}

	// Platforms inis:
	{
		if (!GeneratePlatformConfigFiles(InProjectInfo, OutFailReason))
		{
			return false;
		}
	}

	return true;
}

bool GameProjectUtils::GeneratePlatformConfigFiles(const FProjectInformation& InProjectInfo, FText& OutFailReason)
{
	return true;
}

bool GameProjectUtils::GenerateBasicSourceCode(TArray<FString>& OutCreatedFiles, FText& OutFailReason)
{
	TArray<FString> StartupModuleNames;
	if (GameProjectUtils::GenerateBasicSourceCode(FPaths::GameSourceDir().LeftChop(1), FApp::GetProjectName(), FPaths::ProjectDir(), StartupModuleNames, OutCreatedFiles, OutFailReason))
	{
		GameProjectUtils::UpdateProject(
			FProjectDescriptorModifier::CreateLambda(
			[&StartupModuleNames](FProjectDescriptor& Descriptor)
			{
				return UpdateStartupModuleNames(Descriptor, &StartupModuleNames);
			}));
		return true;
	}

	return false;
}

bool GameProjectUtils::GenerateBasicSourceCode(const FString& NewProjectSourcePath, const FString& NewProjectName, const FString& NewProjectRoot, TArray<FString>& OutGeneratedStartupModuleNames, TArray<FString>& OutCreatedFiles, FText& OutFailReason)
{
	const FString GameModulePath = NewProjectSourcePath / NewProjectName;
	const FString EditorName = NewProjectName + TEXT("Editor");

	// MyGame.Build.cs
	{
		const FString NewBuildFilename = GameModulePath / NewProjectName + TEXT(".Build.cs");
		TArray<FString> PublicDependencyModuleNames;
		PublicDependencyModuleNames.Add(TEXT("Core"));
		PublicDependencyModuleNames.Add(TEXT("CoreUObject"));
		PublicDependencyModuleNames.Add(TEXT("Engine"));
		PublicDependencyModuleNames.Add(TEXT("InputCore"));
		TArray<FString> PrivateDependencyModuleNames;
		if ( GenerateGameModuleBuildFile(NewBuildFilename, NewProjectName, PublicDependencyModuleNames, PrivateDependencyModuleNames, OutFailReason) )
		{
			OutGeneratedStartupModuleNames.Add(NewProjectName);
			OutCreatedFiles.Add(NewBuildFilename);
		}
		else
		{
			return false;
		}
	}

	// MyGame.Target.cs
	{
		const FString NewTargetFilename = NewProjectSourcePath / NewProjectName + TEXT(".Target.cs");
		TArray<FString> ExtraModuleNames;
		ExtraModuleNames.Add( NewProjectName );
		if ( GenerateGameModuleTargetFile(NewTargetFilename, NewProjectName, ExtraModuleNames, OutFailReason) )
		{
			OutCreatedFiles.Add(NewTargetFilename);
		}
		else
		{
			return false;
		}
	}

	// MyGameEditor.Target.cs
	{
		const FString NewTargetFilename = NewProjectSourcePath / EditorName + TEXT(".Target.cs");
		// Include the MyGame module...
		TArray<FString> ExtraModuleNames;
		ExtraModuleNames.Add(NewProjectName);
		if ( GenerateEditorModuleTargetFile(NewTargetFilename, EditorName, ExtraModuleNames, OutFailReason) )
		{
			OutCreatedFiles.Add(NewTargetFilename);
		}
		else
		{
			return false;
		}
	}

	// MyGame.h
	{
		const FString NewHeaderFilename = GameModulePath / NewProjectName + TEXT(".h");
		TArray<FString> PublicHeaderIncludes;
		if ( GenerateGameModuleHeaderFile(NewHeaderFilename, PublicHeaderIncludes, OutFailReason) )
		{
			OutCreatedFiles.Add(NewHeaderFilename);
		}
		else
		{
			return false;
		}
	}

	// MyGame.cpp
	{
		const FString NewCPPFilename = GameModulePath / NewProjectName + TEXT(".cpp");
		if ( GenerateGameModuleCPPFile(NewCPPFilename, NewProjectName, NewProjectName, OutFailReason) )
		{
			OutCreatedFiles.Add(NewCPPFilename);
		}
		else
		{
			return false;
		}
	}

	return true;
}

bool GameProjectUtils::GenerateGameFrameworkSourceCode(const FString& NewProjectSourcePath, const FString& NewProjectName, TArray<FString>& OutCreatedFiles, FText& OutFailReason)
{
	const FString GameModulePath = NewProjectSourcePath / NewProjectName;

	// Used to override the code generation validation since the module we're creating isn't the same as the project we currently have loaded
	FModuleContextInfo NewModuleInfo;
	NewModuleInfo.ModuleName = NewProjectName;
	NewModuleInfo.ModuleType = EHostType::Runtime;
	NewModuleInfo.ModuleSourcePath = FPaths::ConvertRelativePathToFull(GameModulePath / ""); // Ensure trailing /

	// MyGameGameMode.h
	{
		const UClass* BaseClass = AGameModeBase::StaticClass();
		const FString NewClassName = NewProjectName + BaseClass->GetName();
		const FString NewHeaderFilename = GameModulePath / NewClassName + TEXT(".h");
		FString UnusedSyncLocation;
		if ( GenerateClassHeaderFile(NewHeaderFilename, NewClassName, FNewClassInfo(BaseClass), TArray<FString>(), TEXT(""), TEXT(""), UnusedSyncLocation, NewModuleInfo, false, OutFailReason) )
		{
			OutCreatedFiles.Add(NewHeaderFilename);
		}
		else
		{
			return false;
		}
	}

	// MyGameGameMode.cpp
	{
		const UClass* BaseClass = AGameModeBase::StaticClass();
		const FString NewClassName = NewProjectName + BaseClass->GetName();
		const FString NewCPPFilename = GameModulePath / NewClassName + TEXT(".cpp");

		TArray<FString> PropertyOverrides;
		TArray<FString> AdditionalIncludes;
		FString UnusedSyncLocation;

		if ( GenerateClassCPPFile(NewCPPFilename, NewClassName, FNewClassInfo(BaseClass), AdditionalIncludes, PropertyOverrides, TEXT(""), UnusedSyncLocation, NewModuleInfo, OutFailReason) )
		{
			OutCreatedFiles.Add(NewCPPFilename);
		}
		else
		{
			return false;
		}
	}

	return true;
}

bool GameProjectUtils::BuildCodeProject(const FString& ProjectFilename)
{
	// Build the project while capturing the log output. Passing GWarn to CompileGameProject will allow Slate to display the progress bar.
	FStringOutputDevice OutputLog;
	OutputLog.SetAutoEmitLineTerminator(true);
	GLog->AddOutputDevice(&OutputLog);
	bool bCompileSucceeded = FDesktopPlatformModule::Get()->CompileGameProject(FPaths::RootDir(), ProjectFilename, GWarn);
	GLog->RemoveOutputDevice(&OutputLog);

	// Try to compile the modules
	if(!bCompileSucceeded)
	{
		FText DevEnvName = FSourceCodeNavigation::GetSelectedSourceCodeIDE();

		TArray<FText> CompileFailedButtons;
		int32 OpenIDEButton = CompileFailedButtons.Add(FText::Format(LOCTEXT("CompileFailedOpenIDE", "Open with {0}"), DevEnvName));
		CompileFailedButtons.Add(LOCTEXT("CompileFailedCancel", "Cancel"));

		FText LogText = FText::FromString(OutputLog.Replace(LINE_TERMINATOR, TEXT("\n")).TrimEnd());
		int32 CompileFailedChoice = SOutputLogDialog::Open(LOCTEXT("CompileFailedTitle", "Compile Failed"), FText::Format(LOCTEXT("CompileFailedHeader", "The project could not be compiled. Would you like to open it in {0}?"), DevEnvName), LogText, FText::GetEmpty(), CompileFailedButtons);

		FText FailReason;
		if(CompileFailedChoice == OpenIDEButton && !GameProjectUtils::OpenCodeIDE(ProjectFilename, FailReason))
		{
			FMessageDialog::Open(EAppMsgType::Ok, FailReason);
		}
	}
	return bCompileSucceeded;
}

bool GameProjectUtils::GenerateCodeProjectFiles(const FString& ProjectFilename, FText& OutFailReason, FText& OutFailLog)
{
	FStringOutputDevice OutputLog;
	OutputLog.SetAutoEmitLineTerminator(true);
	GLog->AddOutputDevice(&OutputLog);
	bool bHaveProjectFiles = FDesktopPlatformModule::Get()->GenerateProjectFiles(FPaths::RootDir(), ProjectFilename, GWarn);
	GLog->RemoveOutputDevice(&OutputLog);

	if ( !bHaveProjectFiles )
	{
		OutFailReason = LOCTEXT("ErrorWhileGeneratingProjectFiles", "An error occurred while trying to generate project files.");
		OutFailLog = FText::FromString(OutputLog);
		return false;
	}

	return true;
}

int32 FindSpecificBuildFiles(const TCHAR* Path, TMap<FString, FString>& BuildFiles)
{
	class FBuildFileVistor : public IPlatformFile::FDirectoryVisitor
	{
	public:
		TMap<FString, FString>& BuildFiles;
		int32& NumBuildFilesFound;
		FString CheckFilename;
		bool bSawAnyBuildFile;

		FBuildFileVistor(TMap<FString, FString>& InBuildFiles, int32& InNumBuildFilesFound) :
			BuildFiles(InBuildFiles),
			NumBuildFilesFound(InNumBuildFilesFound),
			bSawAnyBuildFile(false)
		{
		}

		virtual bool Visit(const TCHAR* FilenameOrDirectory, bool bIsDirectory)
		{
			if (bIsDirectory)
			{
				return true;
			}

			static const FString BuildFileSuffix(TEXT(".Build.cs"));
			CheckFilename = FilenameOrDirectory;
			if (!CheckFilename.EndsWith(BuildFileSuffix))
			{
				return true;
			}

			bSawAnyBuildFile = true;

			CheckFilename = FPaths::GetCleanFilename(CheckFilename);
			FString* Found = BuildFiles.Find(CheckFilename);
			if (Found && Found->Len() == 0)
			{
				*Found = FilenameOrDirectory;
				++NumBuildFilesFound;
			}

			return false;
		}
	};

	class FBuildDirectoryVistor : public IPlatformFile::FDirectoryVisitor
	{
	public:
		FBuildFileVistor FileVisitor;
		FString CheckDirectory;

		FBuildDirectoryVistor(TMap<FString, FString>& InBuildFiles, int32& InNumBuildFilesFound) :
			FileVisitor(InBuildFiles, InNumBuildFilesFound)
		{
		}

		virtual bool Visit(const TCHAR* FilenameOrDirectory, bool bIsDirectory)
		{
			if (bIsDirectory)
			{
				static const FString Dot(TEXT("."));
				CheckDirectory = FPaths::GetCleanFilename(FilenameOrDirectory);
				if (!CheckDirectory.StartsWith(Dot) && CheckDirectory != TEXT("Public") && CheckDirectory != TEXT("Private"))
				{
					return ScanDirectory(FilenameOrDirectory);
				}
			}

			return true;
		}

		bool ScanDirectory(const TCHAR* FilenameOrDirectory)
		{
			// Iterate files only the first time
			FileVisitor.bSawAnyBuildFile = false;
			IFileManager::Get().IterateDirectory(FilenameOrDirectory, FileVisitor);
			if (!FileVisitor.bSawAnyBuildFile && FileVisitor.NumBuildFilesFound < FileVisitor.BuildFiles.Num())
			{
				// Iterate directories only the second time
				IFileManager::Get().IterateDirectory(FilenameOrDirectory, *this);
			}

			return FileVisitor.NumBuildFilesFound < FileVisitor.BuildFiles.Num();
		}
	};

	int32 NumBuildFilesFound = 0;
	FBuildDirectoryVistor DirectoryVisitor(BuildFiles, NumBuildFilesFound);
	DirectoryVisitor.ScanDirectory(Path);
	return NumBuildFilesFound;
}

void GameProjectUtils::ResetCurrentProjectModulesCache()
{
	IProjectManager::Get().GetCurrentProjectModuleContextInfos().Reset();
}

const TArray<FModuleContextInfo>& GameProjectUtils::GetCurrentProjectModules()
{
	const FProjectDescriptor* const CurrentProject = IProjectManager::Get().GetCurrentProject();
	check(CurrentProject);

	TArray<FModuleContextInfo>& RetModuleInfos = IProjectManager::Get().GetCurrentProjectModuleContextInfos();
	if (RetModuleInfos.Num() > 0)
	{
		return RetModuleInfos;
	}

	RetModuleInfos.Reset(CurrentProject->Modules.Num() + 1);
	if (!GameProjectUtils::ProjectHasCodeFiles() || CurrentProject->Modules.Num() == 0)
	{
		// If this project doesn't currently have any code in it, we need to add a dummy entry for the game
		// so that we can still use the class wizard (this module will be created once we add a class)
		FModuleContextInfo ModuleInfo;
		ModuleInfo.ModuleName = FApp::GetProjectName();
		ModuleInfo.ModuleType = EHostType::Runtime;
		ModuleInfo.ModuleSourcePath = FPaths::ConvertRelativePathToFull(FPaths::GameSourceDir() / ModuleInfo.ModuleName / ""); // Ensure trailing /
		RetModuleInfos.Emplace(ModuleInfo);
	}

	TMap<FString, FString> BuildFilePathsByName;
	BuildFilePathsByName.Reserve(CurrentProject->Modules.Num());
	for (const FModuleDescriptor& ModuleDesc : CurrentProject->Modules)
	{
		BuildFilePathsByName.Add(ModuleDesc.Name.ToString() + TEXT(".Build.cs"));
	}

	FindSpecificBuildFiles(*FPaths::GameSourceDir(), BuildFilePathsByName);

	// Resolve out the paths for each module and add the cut-down into to our output array
	for (const FModuleDescriptor& ModuleDesc : CurrentProject->Modules)
	{
		FModuleContextInfo ModuleInfo;
		ModuleInfo.ModuleName = ModuleDesc.Name.ToString();
		ModuleInfo.ModuleType = ModuleDesc.Type;

		// Try and find the .Build.cs file for this module within our currently loaded project's Source directory
		FString* FullPath = BuildFilePathsByName.Find(ModuleInfo.ModuleName + TEXT(".Build.cs"));
		if (!FullPath)
		{
			continue;
		}

		// Chop the .Build.cs file off the end of the path
		ModuleInfo.ModuleSourcePath = FPaths::GetPath(*FullPath);
		ModuleInfo.ModuleSourcePath = FPaths::ConvertRelativePathToFull(ModuleInfo.ModuleSourcePath / ""); // Ensure trailing /

		RetModuleInfos.Emplace(ModuleInfo);
	}

	return RetModuleInfos;
}

TArray<FModuleContextInfo> GameProjectUtils::GetCurrentProjectPluginModules()
{
	const FProjectDescriptor* const CurrentProject = IProjectManager::Get().GetCurrentProject();
	check(CurrentProject);

	TArray<FModuleContextInfo> RetModuleInfos;

	if (!GameProjectUtils::ProjectHasCodeFiles() || CurrentProject->Modules.Num() == 0)
	{
		// Don't get plugins if the game project has no source tree.
		return RetModuleInfos;
	}

	// Resolve out the paths for each module and add the cut-down into to our output array
	for (const auto& Plugin : IPluginManager::Get().GetDiscoveredPlugins())
	{
		// Only get plugins that are a part of the game project
		if (Plugin->GetLoadedFrom() == EPluginLoadedFrom::Project)
		{
			for (const auto& PluginModule : Plugin->GetDescriptor().Modules)
			{
				FModuleContextInfo ModuleInfo;
				ModuleInfo.ModuleName = PluginModule.Name.ToString();
				ModuleInfo.ModuleType = PluginModule.Type;

				// Try and find the .Build.cs file for this module within the plugin source tree
				FString TmpPath;
				if (!FindSourceFileInProject(ModuleInfo.ModuleName + ".Build.cs", Plugin->GetBaseDir(), TmpPath))
				{
					continue;
				}

				// Chop the .Build.cs file off the end of the path
				ModuleInfo.ModuleSourcePath = FPaths::GetPath(TmpPath);
				ModuleInfo.ModuleSourcePath = FPaths::ConvertRelativePathToFull(ModuleInfo.ModuleSourcePath / ""); // Ensure trailing /

				RetModuleInfos.Emplace(ModuleInfo);
			}
		}
	}

	return RetModuleInfos;
}

bool GameProjectUtils::IsValidSourcePath(const FString& InPath, const FModuleContextInfo& ModuleInfo, FText* const OutFailReason)
{
	const FString AbsoluteInPath = FPaths::ConvertRelativePathToFull(InPath) / ""; // Ensure trailing /

	// Validate the path contains no invalid characters
	if(!FPaths::ValidatePath(AbsoluteInPath, OutFailReason))
	{
		return false;
	}

	if(!AbsoluteInPath.StartsWith(ModuleInfo.ModuleSourcePath))
	{
		if(OutFailReason)
		{
			FFormatNamedArguments Args;
			Args.Add(TEXT("ModuleName"), FText::FromString(ModuleInfo.ModuleName));
			Args.Add(TEXT("RootSourcePath"), FText::FromString(ModuleInfo.ModuleSourcePath));
			*OutFailReason = FText::Format( LOCTEXT("SourcePathInvalidForModule", "All source code for '{ModuleName}' must exist within '{RootSourcePath}'"), Args );
		}
		return false;
	}

	return true;
}

bool GameProjectUtils::CalculateSourcePaths(const FString& InPath, const FModuleContextInfo& ModuleInfo, FString& OutHeaderPath, FString& OutSourcePath, FText* const OutFailReason)
{
	const FString AbsoluteInPath = FPaths::ConvertRelativePathToFull(InPath) / ""; // Ensure trailing /
	OutHeaderPath = AbsoluteInPath;
	OutSourcePath = AbsoluteInPath;

	EClassLocation ClassPathLocation = EClassLocation::UserDefined;
	if(!GetClassLocation(InPath, ModuleInfo, ClassPathLocation, OutFailReason))
	{
		return false;
	}

	const FString RootPath = ModuleInfo.ModuleSourcePath;
	const FString PublicPath = RootPath / "Public" / "";		// Ensure trailing /
	const FString PrivatePath = RootPath / "Private" / "";		// Ensure trailing /
	const FString ClassesPath = RootPath / "Classes" / "";		// Ensure trailing /

	// The root path must exist; we will allow the creation of sub-folders, but not the module root!
	// We ignore this check if the project doesn't already have source code in it, as the module folder won't yet have been created
	const bool bHasCodeFiles = GameProjectUtils::ProjectHasCodeFiles();
	if(!IFileManager::Get().DirectoryExists(*RootPath) && bHasCodeFiles)
	{
		if(OutFailReason)
		{
			FFormatNamedArguments Args;
			Args.Add(TEXT("ModuleSourcePath"), FText::FromString(RootPath));
			*OutFailReason = FText::Format(LOCTEXT("SourcePathMissingModuleRoot", "The specified module path does not exist on disk: {ModuleSourcePath}"), Args);
		}
		return false;
	}

	// The rules for placing header files are as follows:
	// 1) If InPath is the source root, and GetClassLocation has said the class header should be in the Public folder, put it in the Public folder
	// 2) Otherwise, just place the header at InPath (the default set above)
	if(AbsoluteInPath == RootPath)
	{
		OutHeaderPath = (ClassPathLocation == EClassLocation::Public) ? PublicPath : AbsoluteInPath;
	}

	// The rules for placing source files are as follows:
	// 1) If InPath is the source root, and GetClassLocation has said the class header should be in the Public folder, put the source file in the Private folder
	// 2) If InPath is contained within the Public or Classes folder of this module, place it in the equivalent path in the Private folder
	// 3) Otherwise, just place the source file at InPath (the default set above)
	if(AbsoluteInPath == RootPath)
	{
		OutSourcePath = (ClassPathLocation == EClassLocation::Public) ? PrivatePath : AbsoluteInPath;
	}
	else if(ClassPathLocation == EClassLocation::Public)
	{
		OutSourcePath = AbsoluteInPath.Replace(*PublicPath, *PrivatePath);
	}
	else if(ClassPathLocation == EClassLocation::Classes)
	{
		OutSourcePath = AbsoluteInPath.Replace(*ClassesPath, *PrivatePath);
	}

	return !OutHeaderPath.IsEmpty() && !OutSourcePath.IsEmpty();
}

bool GameProjectUtils::GetClassLocation(const FString& InPath, const FModuleContextInfo& ModuleInfo, EClassLocation& OutClassLocation, FText* const OutFailReason)
{
	const FString AbsoluteInPath = FPaths::ConvertRelativePathToFull(InPath) / ""; // Ensure trailing /
	OutClassLocation = EClassLocation::UserDefined;

	if(!IsValidSourcePath(InPath, ModuleInfo, OutFailReason))
	{
		return false;
	}

	const FString RootPath = ModuleInfo.ModuleSourcePath;
	const FString PublicPath = RootPath / "Public" / "";		// Ensure trailing /
	const FString PrivatePath = RootPath / "Private" / "";		// Ensure trailing /
	const FString ClassesPath = RootPath / "Classes" / "";		// Ensure trailing /

	// If either the Public or Private path exists, and we're in the root, force the header/source file to use one of these folders
	const bool bPublicPathExists = IFileManager::Get().DirectoryExists(*PublicPath);
	const bool bPrivatePathExists = IFileManager::Get().DirectoryExists(*PrivatePath);
	const bool bForceInternalPath = AbsoluteInPath == RootPath && (bPublicPathExists || bPrivatePathExists);

	if(AbsoluteInPath == RootPath)
	{
		OutClassLocation = (bPublicPathExists || bForceInternalPath) ? EClassLocation::Public : EClassLocation::UserDefined;
	}
	else if(AbsoluteInPath.StartsWith(PublicPath))
	{
		OutClassLocation = EClassLocation::Public;
	}
	else if(AbsoluteInPath.StartsWith(PrivatePath))
	{
		OutClassLocation = EClassLocation::Private;
	}
	else if(AbsoluteInPath.StartsWith(ClassesPath))
	{
		OutClassLocation = EClassLocation::Classes;
	}
	else
	{
		OutClassLocation = EClassLocation::UserDefined;
	}

	return true;
}

GameProjectUtils::EProjectDuplicateResult GameProjectUtils::DuplicateProjectForUpgrade( const FString& InProjectFile, FString& OutNewProjectFile )
{
	IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();

	// Get the directory part of the project name
	FString OldDirectoryName = FPaths::GetPath(InProjectFile);
	FPaths::NormalizeDirectoryName(OldDirectoryName);
	FString NewDirectoryName = OldDirectoryName;

	// Strip off any previous version number from the project name
	for(int32 LastSpace; NewDirectoryName.FindLastChar(' ', LastSpace); )
	{
		const TCHAR *End = *NewDirectoryName + LastSpace + 1;
		if(End[0] != '4' || End[1] != '.' || !FChar::IsDigit(End[2]))
		{
			break;
		}

		End += 3;

		while(FChar::IsDigit(*End))
		{
			End++;
		}

		if(*End != 0)
		{
			break;
		}

		NewDirectoryName.LeftInline(LastSpace, EAllowShrinking::No);
		NewDirectoryName.TrimEndInline();
	}

	// Append the new version number
	NewDirectoryName += FString::Printf(TEXT(" %s"), *FEngineVersion::Current().ToString(EVersionComponent::Minor));

	// Find a directory name that doesn't exist
	FString BaseDirectoryName = NewDirectoryName;
	for(int32 Idx = 2; IFileManager::Get().DirectoryExists(*NewDirectoryName); Idx++)
	{
		NewDirectoryName = FString::Printf(TEXT("%s - %d"), *BaseDirectoryName, Idx);
	}

	// Recursively find all the files we need to copy, excluding those that are within the directories listed in SourceDirectoriesToSkip
	struct FGatherFilesToCopyHelper
	{
	public:
		FGatherFilesToCopyHelper(FString InRootSourceDirectory)
			: RootSourceDirectory(MoveTemp(InRootSourceDirectory))
		{
			static const FString RelativeDirectoriesToSkip[] = {
				TEXT("Binaries"),
				TEXT("DerivedDataCache"),
				TEXT("Intermediate"),
				TEXT("Saved/Autosaves"),
				TEXT("Saved/Backup"),
				TEXT("Saved/Cooked"),
				TEXT("Saved/Config"),
				TEXT("Saved/HardwareSurvey"),
				TEXT("Saved/Logs"),
				TEXT("Saved/StagedBuilds"),
			};

			SourceDirectoriesToSkip.Reserve(UE_ARRAY_COUNT(RelativeDirectoriesToSkip));
			for (const FString& RelativeDirectoryToSkip : RelativeDirectoriesToSkip)
			{
				SourceDirectoriesToSkip.Emplace(RootSourceDirectory / RelativeDirectoryToSkip);
			}
		}

		void GatherFilesToCopy(TArray<FString>& OutSourceDirectories, TArray<FString>& OutSourceFiles)
		{
			GatherFilesToCopy(RootSourceDirectory, OutSourceDirectories, OutSourceFiles);
		}

	private:
		void GatherFilesToCopy(const FString& InSourceDirectoryPath, TArray<FString>& OutSourceDirectories, TArray<FString>& OutSourceFiles)
		{
			const FString SourceDirectorySearchWildcard = InSourceDirectoryPath / TEXT("*");

			OutSourceDirectories.Emplace(InSourceDirectoryPath);

			TArray<FString> SourceFilenames;
			IFileManager::Get().FindFiles(SourceFilenames, *SourceDirectorySearchWildcard, true, false);

			OutSourceFiles.Reserve(OutSourceFiles.Num() + SourceFilenames.Num());
			for (const FString& SourceFilename : SourceFilenames)
			{
				OutSourceFiles.Emplace(InSourceDirectoryPath / SourceFilename);
			}

			TArray<FString> SourceSubDirectoryNames;
			IFileManager::Get().FindFiles(SourceSubDirectoryNames, *SourceDirectorySearchWildcard, false, true);

			for (const FString& SourceSubDirectoryName : SourceSubDirectoryNames)
			{
				const FString SourceSubDirectoryPath = InSourceDirectoryPath / SourceSubDirectoryName;
				if (!SourceDirectoriesToSkip.Contains(SourceSubDirectoryPath))
				{
					GatherFilesToCopy(SourceSubDirectoryPath, OutSourceDirectories, OutSourceFiles);
				}
			}
		}

		FString RootSourceDirectory;
		TArray<FString> SourceDirectoriesToSkip;
	};

	TArray<FString> SourceDirectories;
	TArray<FString> SourceFiles;
	FGatherFilesToCopyHelper(OldDirectoryName).GatherFilesToCopy(SourceDirectories, SourceFiles);

	// Copy everything
	bool bCopySucceeded = true;
	bool bUserCanceled = false;
	GWarn->BeginSlowTask(LOCTEXT("CreatingCopyOfProject", "Creating copy of project..."), true, true);
	for(int32 Idx = 0; Idx < SourceDirectories.Num() && bCopySucceeded; Idx++)
	{
		FString TargetDirectory = NewDirectoryName + SourceDirectories[Idx].Mid(OldDirectoryName.Len());
		bUserCanceled = GWarn->ReceivedUserCancel();
		bCopySucceeded = !bUserCanceled && PlatformFile.CreateDirectory(*TargetDirectory);
		GWarn->UpdateProgress(Idx + 1, SourceDirectories.Num() + SourceFiles.Num());
	}
	for(int32 Idx = 0; Idx < SourceFiles.Num() && bCopySucceeded; Idx++)
	{
		FString TargetFile = NewDirectoryName + SourceFiles[Idx].Mid(OldDirectoryName.Len());
		bUserCanceled = GWarn->ReceivedUserCancel();
		bCopySucceeded =  !bUserCanceled && PlatformFile.CopyFile(*TargetFile, *SourceFiles[Idx]);
		GWarn->UpdateProgress(SourceDirectories.Num() + Idx + 1, SourceDirectories.Num() + SourceFiles.Num());
	}
	GWarn->EndSlowTask();

	// Wipe the directory if the user canceled or we couldn't update
	if(!bCopySucceeded)
	{
		PlatformFile.DeleteDirectoryRecursively(*NewDirectoryName);
		if(bUserCanceled)
		{
			return EProjectDuplicateResult::UserCanceled;
		}
		else
		{
			return EProjectDuplicateResult::Failed;
		}
	}

	// Otherwise fixup the output project filename
	OutNewProjectFile = NewDirectoryName / FPaths::GetCleanFilename(InProjectFile);
	return EProjectDuplicateResult::Succeeded;
}

void GameProjectUtils::UpdateSupportedTargetPlatforms(const FName& InPlatformName, const bool bIsSupported)
{
	const FString& ProjectFilename = FPaths::GetProjectFilePath();
	if (!ProjectFilename.IsEmpty())
	{
		// First attempt to check out the file if SCC is enabled
		if (ISourceControlModule::Get().IsEnabled())
		{
			FText UnusedFailReason;
			CheckoutGameProjectFile(ProjectFilename, UnusedFailReason);
		}

		// Second make sure the file is writable
		if (FPlatformFileManager::Get().GetPlatformFile().IsReadOnly(*ProjectFilename))
		{
			FPlatformFileManager::Get().GetPlatformFile().SetReadOnly(*ProjectFilename, false);
		}

		IProjectManager::Get().UpdateSupportedTargetPlatformsForCurrentProject(InPlatformName, bIsSupported);
	}
}

void GameProjectUtils::ClearSupportedTargetPlatforms()
{
	const FString& ProjectFilename = FPaths::GetProjectFilePath();
	if (!ProjectFilename.IsEmpty())
	{
		// First attempt to check out the file if SCC is enabled
		if (ISourceControlModule::Get().IsEnabled())
		{
			FText UnusedFailReason;
			CheckoutGameProjectFile(ProjectFilename, UnusedFailReason);
		}

		// Second make sure the file is writable
		if (FPlatformFileManager::Get().GetPlatformFile().IsReadOnly(*ProjectFilename))
		{
			FPlatformFileManager::Get().GetPlatformFile().SetReadOnly(*ProjectFilename, false);
		}

		IProjectManager::Get().ClearSupportedTargetPlatformsForCurrentProject();
	}
}

bool GameProjectUtils::UpdateAdditionalPluginDirectory(const FString& InDir, const bool bAddOrRemove)
{
	const FString& ProjectFilename = FPaths::GetProjectFilePath();
	if (!ProjectFilename.IsEmpty())
	{
		// First attempt to check out the file if SCC is enabled
		if (ISourceControlModule::Get().IsEnabled())
		{
			FText UnusedFailReason;
			CheckoutGameProjectFile(ProjectFilename, UnusedFailReason);
		}

		// Second make sure the file is writable
		if (FPlatformFileManager::Get().GetPlatformFile().IsReadOnly(*ProjectFilename))
		{
			FPlatformFileManager::Get().GetPlatformFile().SetReadOnly(*ProjectFilename, false);
		}

		return IProjectManager::Get().UpdateAdditionalPluginDirectory(InDir, bAddOrRemove);
	}

	return false;
}

const TCHAR* GameProjectUtils::GetDefaultBuildSettingsVersion()
{
	return TEXT("BuildSettingsVersion.V7");
}

bool GameProjectUtils::ReadTemplateFile(const FString& TemplateFileName, FString& OutFileContents, FText& OutFailReason)
{
	const FString FullFileName = UClassTemplateEditorSubsystem::GetEngineTemplateDirectory() / TemplateFileName;
	if ( FFileHelper::LoadFileToString(OutFileContents, *FullFileName) )
	{
		return true;
	}

	FFormatNamedArguments Args;
	Args.Add( TEXT("FullFileName"), FText::FromString( FullFileName ) );
	OutFailReason = FText::Format( LOCTEXT("FailedToReadTemplateFile", "Failed to read template file \"{FullFileName}\""), Args );
	return false;
}

bool GameProjectUtils::WriteOutputFile(const FString& OutputFilename, const FString& OutputFileContents, FText& OutFailReason)
{
	if ( FFileHelper::SaveStringToFile(OutputFileContents, *OutputFilename ) )
	{
		return true;
	}

	FFormatNamedArguments Args;
	Args.Add( TEXT("OutputFilename"), FText::FromString( OutputFilename ) );
	OutFailReason = FText::Format( LOCTEXT("FailedToWriteOutputFile", "Failed to write output file \"{OutputFilename}\". Perhaps the file is Read-Only?"), Args );
	return false;
}

FString GameProjectUtils::MakeCopyrightLine()
{
	const FString CopyrightNotice = GetDefault<UGeneralProjectSettings>()->CopyrightNotice;
	if (!CopyrightNotice.IsEmpty())
	{
		return FString(TEXT("// ")) + CopyrightNotice;
	}
	else
	{
		return FString();
	}
}

FString GameProjectUtils::MakeCommaDelimitedList(const TArray<FString>& InList, bool bPlaceQuotesAroundEveryElement)
{
	FString ReturnString;

	for ( auto ListIt = InList.CreateConstIterator(); ListIt; ++ListIt )
	{
		FString ElementStr;
		if ( bPlaceQuotesAroundEveryElement )
		{
			ElementStr = FString::Printf( TEXT("\"%s\""), **ListIt);
		}
		else
		{
			ElementStr = *ListIt;
		}

		if ( ReturnString.Len() > 0 )
		{
			// If this is not the first item in the list, prepend with a comma
			ElementStr = FString::Printf(TEXT(", %s"), *ElementStr);
		}

		ReturnString += ElementStr;
	}

	return ReturnString;
}

FString GameProjectUtils::MakeIncludeList(const TArray<FString>& InList)
{
	FString ReturnString;

	for ( auto ListIt = InList.CreateConstIterator(); ListIt; ++ListIt )
	{
		ReturnString += FString::Printf(IncludePathFormatString, **ListIt);
		ReturnString += LINE_TERMINATOR;
	}

	return ReturnString;
}

FString GameProjectUtils::DetermineModuleIncludePath(const FModuleContextInfo& ModuleInfo, const FString& FileRelativeTo)
{
	FString ModuleIncludePath;

	if(FindSourceFileInProject(ModuleInfo.ModuleName + ".h", ModuleInfo.ModuleSourcePath, ModuleIncludePath))
	{
		// Work out where the module header is;
		// if it's Public then we can include it without any path since all Public and Classes folders are on the include path
		// if it's located elsewhere, then we'll need to include it relative to the module source root as we can't guarantee
		// that other folders are on the include paths
		EClassLocation ModuleLocation;
		if(GetClassLocation(ModuleIncludePath, ModuleInfo, ModuleLocation))
		{
			if(ModuleLocation == EClassLocation::Public || ModuleLocation == EClassLocation::Classes)
			{
				ModuleIncludePath = ModuleInfo.ModuleName + ".h";
			}
			else
			{
				// If the path to our new class is the same as the path to the module, we can include it directly
				const FString ModulePath = FPaths::ConvertRelativePathToFull(FPaths::GetPath(ModuleIncludePath));
				const FString ClassPath = FPaths::ConvertRelativePathToFull(FPaths::GetPath(FileRelativeTo));
				if(ModulePath == ClassPath)
				{
					ModuleIncludePath = ModuleInfo.ModuleName + ".h";
				}
				else
				{
					// Updates ModuleIncludePath internally
					if(!FPaths::MakePathRelativeTo(ModuleIncludePath, *ModuleInfo.ModuleSourcePath))
					{
						// Failed; just assume we can include it without any relative path
						ModuleIncludePath = ModuleInfo.ModuleName + ".h";
					}
				}
			}
		}
		else
		{
			// Failed; just assume we can include it without any relative path
			ModuleIncludePath = ModuleInfo.ModuleName + ".h";
		}
	}
	else
	{
		// This could potentially fail when generating new projects if the module file hasn't yet been created; just assume we can include it without any relative path
		ModuleIncludePath = ModuleInfo.ModuleName + ".h";
	}

	return ModuleIncludePath;
}

/**
 * Generates UObject class constructor definition with property overrides.
 *
 * @param Out String to assign generated constructor to.
 * @param PrefixedClassName Prefixed class name for which we generate the constructor.
 * @param PropertyOverridesStr String with property overrides in the constructor.
 * @param OutFailReason Template read function failure reason.
 *
 * @returns True on success. False otherwise.
 */
bool GenerateConstructorDefinition(FString& Out, const FString& PrefixedClassName, const FString& PropertyOverridesStr, FText& OutFailReason)
{
	FString Template;
	if (!GameProjectUtils::ReadTemplateFile(TEXT("UObjectClassConstructorDefinition.template"), Template, OutFailReason))
	{
		return false;
	}

	Out = Template.Replace(TEXT("%PREFIXED_CLASS_NAME%"), *PrefixedClassName, ESearchCase::CaseSensitive);
	Out = Out.Replace(TEXT("%PROPERTY_OVERRIDES%"), *PropertyOverridesStr, ESearchCase::CaseSensitive);

	return true;
}

/**
 * Generates UObject class constructor declaration.
 *
 * @param Out String to assign generated constructor to.
 * @param PrefixedClassName Prefixed class name for which we generate the constructor.
 * @param OutFailReason Template read function failure reason.
 *
 * @returns True on success. False otherwise.
 */
bool GenerateConstructorDeclaration(FString& Out, const FString& PrefixedClassName, FText& OutFailReason)
{
	FString Template;
	if (!GameProjectUtils::ReadTemplateFile(TEXT("UObjectClassConstructorDeclaration.template"), Template, OutFailReason))
	{
		return false;
	}

	Out = Template.Replace(TEXT("%PREFIXED_CLASS_NAME%"), *PrefixedClassName, ESearchCase::CaseSensitive);

	return true;
}

bool GameProjectUtils::GenerateClassHeaderFile(const FString& NewHeaderFileName, const FString UnPrefixedClassName, const FNewClassInfo ParentClassInfo, const TArray<FString>& ClassSpecifierList, const FString& ClassProperties, const FString& ClassFunctionDeclarations, FString& OutSyncLocation, const FModuleContextInfo& ModuleInfo, bool bDeclareConstructor, FText& OutFailReason)
{
	FString Template;
	bool bTemplateFound = false;
	if (GEditor)
	{
		if (UClassTemplateEditorSubsystem* TemplateSubsystem = GEditor->GetEditorSubsystem<UClassTemplateEditorSubsystem>())
		{
			for (const UClass* BaseClass = ParentClassInfo.BaseClass; BaseClass != nullptr; BaseClass = BaseClass->GetSuperClass())
			{
				if (const UClassTemplate* ClassTemplate = TemplateSubsystem->FindClassTemplate(BaseClass))
				{
					bTemplateFound = ClassTemplate->ReadHeader(Template, OutFailReason);
					if (!bTemplateFound)
					{
						return false;
					}
					break;
				}
			}
		}
	}

	if (!bTemplateFound && !ReadTemplateFile(ParentClassInfo.GetHeaderTemplateFilename(), Template, OutFailReason))
	{
		return false;
	}

	const FString ClassPrefix = ParentClassInfo.GetClassPrefixCPP();
	const FString PrefixedClassName = ClassPrefix + UnPrefixedClassName;
	const FString PrefixedBaseClassName = ClassPrefix + ParentClassInfo.GetClassNameCPP();

	FString BaseClassIncludeDirective;
	FString BaseClassIncludePath;
	if(ParentClassInfo.GetIncludePath(BaseClassIncludePath))
	{
		BaseClassIncludeDirective = FString::Printf(IncludePathFormatString, *BaseClassIncludePath);
	}

	FString ModuleAPIMacro;
	{
		EClassLocation ClassPathLocation = EClassLocation::UserDefined;
		if ( GetClassLocation(NewHeaderFileName, ModuleInfo, ClassPathLocation) )
		{
			// If this class isn't Private, make sure and include the API macro so it can be linked within other modules
			if ( ClassPathLocation != EClassLocation::Private )
			{
				ModuleAPIMacro = ModuleInfo.ModuleName.ToUpper() + "_API "; // include a trailing space for the template formatting
			}
		}
	}

	FString EventualConstructorDeclaration;
	if (bDeclareConstructor)
	{
		if (!GenerateConstructorDeclaration(EventualConstructorDeclaration, PrefixedClassName, OutFailReason))
		{
			return false;
		}
	}

	// Not all of these will exist in every class template
	FString FinalOutput = Template.Replace(TEXT("%COPYRIGHT_LINE%"), *MakeCopyrightLine(), ESearchCase::CaseSensitive);
	FinalOutput = FinalOutput.Replace(TEXT("%UNPREFIXED_CLASS_NAME%"), *UnPrefixedClassName, ESearchCase::CaseSensitive);
	FinalOutput = FinalOutput.Replace(TEXT("%CLASS_MODULE_API_MACRO%"), *ModuleAPIMacro, ESearchCase::CaseSensitive);
	FinalOutput = FinalOutput.Replace(TEXT("%UCLASS_SPECIFIER_LIST%"), *MakeCommaDelimitedList(ClassSpecifierList, false), ESearchCase::CaseSensitive);
	FinalOutput = FinalOutput.Replace(TEXT("%PREFIXED_CLASS_NAME%"), *PrefixedClassName, ESearchCase::CaseSensitive);
	FinalOutput = FinalOutput.Replace(TEXT("%PREFIXED_BASE_CLASS_NAME%"), *PrefixedBaseClassName, ESearchCase::CaseSensitive);
	FinalOutput = FinalOutput.Replace(TEXT("%MODULE_NAME%"), *ModuleInfo.ModuleName, ESearchCase::CaseSensitive);

	// Special case where where the wildcard starts with a tab and ends with a new line
	const bool bLeadingTab = true;
	const bool bTrailingNewLine = true;
	FinalOutput = ReplaceWildcard(FinalOutput, TEXT("%EVENTUAL_CONSTRUCTOR_DECLARATION%"), *EventualConstructorDeclaration, bLeadingTab, bTrailingNewLine);
	FinalOutput = ReplaceWildcard(FinalOutput, TEXT("%CLASS_PROPERTIES%"), *ClassProperties, bLeadingTab, bTrailingNewLine);
	FinalOutput = ReplaceWildcard(FinalOutput, TEXT("%CLASS_FUNCTION_DECLARATIONS%"), *ClassFunctionDeclarations, bLeadingTab, bTrailingNewLine);
	if (BaseClassIncludeDirective.Len() == 0)
	{
		FinalOutput = FinalOutput.Replace(TEXT("%BASE_CLASS_INCLUDE_DIRECTIVE%") LINE_TERMINATOR, TEXT(""), ESearchCase::CaseSensitive);
	}
	FinalOutput = FinalOutput.Replace(TEXT("%BASE_CLASS_INCLUDE_DIRECTIVE%"), *BaseClassIncludeDirective, ESearchCase::CaseSensitive);

	HarvestCursorSyncLocation( FinalOutput, OutSyncLocation );

	return WriteOutputFile(NewHeaderFileName, FinalOutput, OutFailReason);
}

static bool TryParseIncludeDirective(const FString& Text, int StartPos, int EndPos, FString& IncludePath)
{
	// Check if the line starts with a # character
	int Pos = StartPos;
	while (Pos < EndPos && FChar::IsWhitespace(Text[Pos]))
	{
		Pos++;
	}
	if (Pos == EndPos || Text[Pos++] != '#')
	{
		return false;
	}
	while (Pos < EndPos && FChar::IsWhitespace(Text[Pos]))
	{
		Pos++;
	}

	// Check it's an include directive
	const TCHAR* IncludeText = TEXT("include");
	for (int Idx = 0; IncludeText[Idx] != 0; Idx++)
	{
		if (Pos == EndPos || Text[Pos] != IncludeText[Idx])
		{
			return false;
		}
		Pos++;
	}
	while (Pos < EndPos && FChar::IsWhitespace(Text[Pos]))
	{
		Pos++;
	}

	// Parse out the quoted include path
	if (Pos == EndPos || Text[Pos++] != '"')
	{
		return false;
	}
	int IncludePathPos = Pos;
	while (Pos < EndPos && Text[Pos] != '"')
	{
		Pos++;
	}
	IncludePath = Text.Mid(IncludePathPos, Pos - IncludePathPos);
	return true;
}

static bool IsUsingOldStylePch(FString BaseDir)
{
	// Find all the cpp files under the base directory
	TArray<FString> Files;
	IFileManager::Get().FindFilesRecursive(Files, *BaseDir, TEXT("*.cpp"), true, false, false);

	// Parse the first include directive for up to 16 include paths
	TArray<FString> FirstIncludedFiles;
	for (int Idx = 0; Idx < Files.Num() && Idx < 16; Idx++)
	{
		FString Text;
		FFileHelper::LoadFileToString(Text, *Files[Idx]);

		int LinePos = 0;
		while(LinePos < Text.Len())
		{
			int EndOfLinePos = LinePos;
			while (EndOfLinePos < Text.Len() && Text[EndOfLinePos] != '\n')
			{
				EndOfLinePos++;
			}

			FString IncludePath;
			if (TryParseIncludeDirective(Text, LinePos, EndOfLinePos, IncludePath))
			{
				FirstIncludedFiles.AddUnique(FPaths::GetCleanFilename(IncludePath));
				break;
			}

			LinePos = EndOfLinePos + 1;
		}
	}
	return FirstIncludedFiles.Num() == 1 && Files.Num() > 1;
}

bool GameProjectUtils::GenerateClassCPPFile(const FString& NewCPPFileName, const FString UnPrefixedClassName, const FNewClassInfo ParentClassInfo, const TArray<FString>& AdditionalIncludes, const TArray<FString>& PropertyOverrides, const FString& AdditionalMemberDefinitions, FString& OutSyncLocation, const FModuleContextInfo& ModuleInfo, FText& OutFailReason)
{
	FString Template;
	bool bTemplateFound = false;
	if (GEditor)
	{
		if (UClassTemplateEditorSubsystem* TemplateSubsystem = GEditor->GetEditorSubsystem<UClassTemplateEditorSubsystem>())
		{
			for (const UClass* BaseClass = ParentClassInfo.BaseClass; BaseClass != nullptr; BaseClass = BaseClass->GetSuperClass())
			{
				if (const UClassTemplate* ClassTemplate = TemplateSubsystem->FindClassTemplate(BaseClass))
				{
					bTemplateFound = ClassTemplate->ReadSource(Template, OutFailReason);
					if (!bTemplateFound)
					{
						return false;
					}
					break;
				}
			}
		}
	}

	if (!bTemplateFound && !ReadTemplateFile(ParentClassInfo.GetSourceTemplateFilename(), Template, OutFailReason))
	{
		return false;
	}

	const FString ClassPrefix = ParentClassInfo.GetClassPrefixCPP();
	const FString PrefixedClassName = ClassPrefix + UnPrefixedClassName;
	const FString PrefixedBaseClassName = ClassPrefix + ParentClassInfo.GetClassNameCPP();

	EClassLocation ClassPathLocation = EClassLocation::UserDefined;
	if ( !GetClassLocation(NewCPPFileName, ModuleInfo, ClassPathLocation, &OutFailReason) )
	{
		return false;
	}

	FString PropertyOverridesStr;
	for ( int32 OverrideIdx = 0; OverrideIdx < PropertyOverrides.Num(); ++OverrideIdx )
	{
		if ( OverrideIdx > 0 )
		{
			PropertyOverridesStr += LINE_TERMINATOR;
		}

		PropertyOverridesStr += TEXT("\t");
		PropertyOverridesStr += *PropertyOverrides[OverrideIdx];
	}

	// Calculate the correct include path for the module header
	FString PchIncludeDirective;
	if (IsUsingOldStylePch(ModuleInfo.ModuleSourcePath))
	{
		const FString ModuleIncludePath = DetermineModuleIncludePath(ModuleInfo, NewCPPFileName);
		if (ModuleIncludePath.Len() > 0)
		{
			PchIncludeDirective = FString::Printf(IncludePathFormatString, *ModuleIncludePath);
		}
	}

	FString EventualConstructorDefinition;
	if (PropertyOverrides.Num() != 0)
	{
		if (!GenerateConstructorDefinition(EventualConstructorDefinition, PrefixedClassName, PropertyOverridesStr, OutFailReason))
		{
			return false;
		}
	}

	// Not all of these will exist in every class template
	FString FinalOutput = Template.Replace(TEXT("%COPYRIGHT_LINE%"), *MakeCopyrightLine(), ESearchCase::CaseSensitive);
	FinalOutput = FinalOutput.Replace(TEXT("%UNPREFIXED_CLASS_NAME%"), *UnPrefixedClassName, ESearchCase::CaseSensitive);
	FinalOutput = FinalOutput.Replace(TEXT("%PREFIXED_CLASS_NAME%"), *PrefixedClassName, ESearchCase::CaseSensitive);
	FinalOutput = FinalOutput.Replace(TEXT("%MODULE_NAME%"), *ModuleInfo.ModuleName, ESearchCase::CaseSensitive);
	FinalOutput = FinalOutput.Replace(TEXT("%PCH_INCLUDE_DIRECTIVE%"), *PchIncludeDirective, ESearchCase::CaseSensitive);

	// Fixup the header file include for this cpp file
	{
		const FString RelativeHeaderIncludePath = GetIncludePathForFile(NewCPPFileName, ModuleInfo.ModuleSourcePath);

		// First make sure we remove any potentially incorrect, legacy paths generated from some version of #include "%UNPREFIXED_CLASS_NAME%.h"
		// This didn't take into account potential subfolders for the created class
		const FString LegacyHeaderInclude = FString::Printf(IncludePathFormatString, *FPaths::GetCleanFilename(RelativeHeaderIncludePath));
		FinalOutput = FinalOutput.Replace(*LegacyHeaderInclude, TEXT(""), ESearchCase::CaseSensitive);

		// Now add the correct include directive which may include a subfolder.
		const FString HeaderIncludeDirective = FString::Printf(IncludePathFormatString, *RelativeHeaderIncludePath);
		FinalOutput = FinalOutput.Replace(TEXT("%MY_HEADER_INCLUDE_DIRECTIVE%"), *HeaderIncludeDirective, ESearchCase::CaseSensitive);
	}

	// Special case where where the wildcard ends with a new line
	const bool bLeadingTab = false;
	const bool bTrailingNewLine = true;
	FinalOutput = ReplaceWildcard(FinalOutput, TEXT("%ADDITIONAL_INCLUDE_DIRECTIVES%"), *MakeIncludeList(AdditionalIncludes), bLeadingTab, bTrailingNewLine);
	FinalOutput = ReplaceWildcard(FinalOutput, TEXT("%EVENTUAL_CONSTRUCTOR_DEFINITION%"), *EventualConstructorDefinition, bLeadingTab, bTrailingNewLine);
	FinalOutput = ReplaceWildcard(FinalOutput, TEXT("%ADDITIONAL_MEMBER_DEFINITIONS%"), *AdditionalMemberDefinitions, bLeadingTab, bTrailingNewLine);

	HarvestCursorSyncLocation( FinalOutput, OutSyncLocation );

	return WriteOutputFile(NewCPPFileName, FinalOutput, OutFailReason);
}

bool GameProjectUtils::GenerateGameModuleBuildFile(const FString& NewBuildFileName, const FString& ModuleName, const TArray<FString>& PublicDependencyModuleNames, const TArray<FString>& PrivateDependencyModuleNames, FText& OutFailReason)
{
	FString Template;
	if (!ReadTemplateFile(TEXT("GameModule.Build.cs.template"), Template, OutFailReason))
	{
		return false;
	}

	FString FinalOutput = Template.Replace(TEXT("%COPYRIGHT_LINE%"), *MakeCopyrightLine(), ESearchCase::CaseSensitive);
	FinalOutput = FinalOutput.Replace(TEXT("%PUBLIC_DEPENDENCY_MODULE_NAMES%"), *MakeCommaDelimitedList(PublicDependencyModuleNames), ESearchCase::CaseSensitive);
	FinalOutput = FinalOutput.Replace(TEXT("%PRIVATE_DEPENDENCY_MODULE_NAMES%"), *MakeCommaDelimitedList(PrivateDependencyModuleNames), ESearchCase::CaseSensitive);
	FinalOutput = FinalOutput.Replace(TEXT("%MODULE_NAME%"), *ModuleName, ESearchCase::CaseSensitive);

	ResetCurrentProjectModulesCache();

	return WriteOutputFile(NewBuildFileName, FinalOutput, OutFailReason);
}

bool GameProjectUtils::GeneratePluginModuleBuildFile(const FString& NewBuildFileName, const FString& ModuleName, const TArray<FString>& PublicDependencyModuleNames, const TArray<FString>& PrivateDependencyModuleNames, FText& OutFailReason, bool bUseExplicitOrSharedPCHs/* = true*/)
{
	FString Template;
	if ( !ReadTemplateFile(TEXT("PluginModule.Build.cs.template"), Template, OutFailReason) )
	{
		return false;
	}

	FString FinalOutput = Template.Replace(TEXT("%COPYRIGHT_LINE%"), *MakeCopyrightLine(), ESearchCase::CaseSensitive);
	FinalOutput = FinalOutput.Replace(TEXT("%PUBLIC_DEPENDENCY_MODULE_NAMES%"), *MakeCommaDelimitedList(PublicDependencyModuleNames), ESearchCase::CaseSensitive);
	FinalOutput = FinalOutput.Replace(TEXT("%PRIVATE_DEPENDENCY_MODULE_NAMES%"), *MakeCommaDelimitedList(PrivateDependencyModuleNames), ESearchCase::CaseSensitive);
	FinalOutput = FinalOutput.Replace(TEXT("%MODULE_NAME%"), *ModuleName, ESearchCase::CaseSensitive);

	const FString PCHUsage = bUseExplicitOrSharedPCHs ? TEXT("UseExplicitOrSharedPCHs") : TEXT("UseSharedPCHs");
	FinalOutput = FinalOutput.Replace(TEXT("%PCH_USAGE%"), *PCHUsage, ESearchCase::CaseSensitive);

	return WriteOutputFile(NewBuildFileName, FinalOutput, OutFailReason);
}

bool GameProjectUtils::GenerateGameModuleTargetFile(const FString& NewBuildFileName, const FString& ModuleName, const TArray<FString>& ExtraModuleNames, FText& OutFailReason)
{
	FString Template;
	if ( !ReadTemplateFile(TEXT("Stub.Target.cs.template"), Template, OutFailReason) )
	{
		return false;
	}

	FString FinalOutput = Template.Replace(TEXT("%COPYRIGHT_LINE%"), *MakeCopyrightLine(), ESearchCase::CaseSensitive);
	FinalOutput = FinalOutput.Replace(TEXT("%EXTRA_MODULE_NAMES%"), *MakeCommaDelimitedList(ExtraModuleNames), ESearchCase::CaseSensitive);
	FinalOutput = FinalOutput.Replace(TEXT("%MODULE_NAME%"), *ModuleName, ESearchCase::CaseSensitive);
	FinalOutput = FinalOutput.Replace(TEXT("%TARGET_TYPE%"), TEXT("Game"), ESearchCase::CaseSensitive);
	FinalOutput = FinalOutput.Replace(TEXT("%DEFAULT_BUILD_SETTINGS_VERSION%"), GetDefaultBuildSettingsVersion(), ESearchCase::CaseSensitive);

	return WriteOutputFile(NewBuildFileName, FinalOutput, OutFailReason);
}

bool GameProjectUtils::GenerateEditorModuleBuildFile(const FString& NewBuildFileName, const FString& ModuleName, const TArray<FString>& PublicDependencyModuleNames, const TArray<FString>& PrivateDependencyModuleNames, FText& OutFailReason)
{
	FString Template;
	if ( !ReadTemplateFile(TEXT("EditorModule.Build.cs.template"), Template, OutFailReason) )
	{
		return false;
	}

	FString FinalOutput = Template.Replace(TEXT("%COPYRIGHT_LINE%"), *MakeCopyrightLine(), ESearchCase::CaseSensitive);
	FinalOutput = FinalOutput.Replace(TEXT("%PUBLIC_DEPENDENCY_MODULE_NAMES%"), *MakeCommaDelimitedList(PublicDependencyModuleNames), ESearchCase::CaseSensitive);
	FinalOutput = FinalOutput.Replace(TEXT("%PRIVATE_DEPENDENCY_MODULE_NAMES%"), *MakeCommaDelimitedList(PrivateDependencyModuleNames), ESearchCase::CaseSensitive);
	FinalOutput = FinalOutput.Replace(TEXT("%MODULE_NAME%"), *ModuleName, ESearchCase::CaseSensitive);

	ResetCurrentProjectModulesCache();

	return WriteOutputFile(NewBuildFileName, FinalOutput, OutFailReason);
}

bool GameProjectUtils::GenerateEditorModuleTargetFile(const FString& NewBuildFileName, const FString& ModuleName, const TArray<FString>& ExtraModuleNames, FText& OutFailReason)
{
	FString Template;
	if ( !ReadTemplateFile(TEXT("Stub.Target.cs.template"), Template, OutFailReason) )
	{
		return false;
	}

	FString FinalOutput = Template.Replace(TEXT("%COPYRIGHT_LINE%"), *MakeCopyrightLine(), ESearchCase::CaseSensitive);
	FinalOutput = FinalOutput.Replace(TEXT("%EXTRA_MODULE_NAMES%"), *MakeCommaDelimitedList(ExtraModuleNames), ESearchCase::CaseSensitive);
	FinalOutput = FinalOutput.Replace(TEXT("%MODULE_NAME%"), *ModuleName, ESearchCase::CaseSensitive);
	FinalOutput = FinalOutput.Replace(TEXT("%TARGET_TYPE%"), TEXT("Editor"), ESearchCase::CaseSensitive);
	FinalOutput = FinalOutput.Replace(TEXT("%DEFAULT_BUILD_SETTINGS_VERSION%"), GetDefaultBuildSettingsVersion(), ESearchCase::CaseSensitive);

	return WriteOutputFile(NewBuildFileName, FinalOutput, OutFailReason);
}

bool GameProjectUtils::GenerateGameModuleCPPFile(const FString& NewBuildFileName, const FString& ModuleName, const FString& GameName, FText& OutFailReason)
{
	FString Template;
	if ( !ReadTemplateFile(TEXT("GameModule.cpp.template"), Template, OutFailReason) )
	{
		return false;
	}

	FString FinalOutput = Template.Replace(TEXT("%COPYRIGHT_LINE%"), *MakeCopyrightLine(), ESearchCase::CaseSensitive);
	FinalOutput = FinalOutput.Replace(TEXT("%MODULE_NAME%"), *ModuleName, ESearchCase::CaseSensitive);
	FinalOutput = FinalOutput.Replace(TEXT("%GAME_NAME%"), *GameName, ESearchCase::CaseSensitive);

	return WriteOutputFile(NewBuildFileName, FinalOutput, OutFailReason);
}

bool GameProjectUtils::GenerateGameModuleHeaderFile(const FString& NewBuildFileName, const TArray<FString>& PublicHeaderIncludes, FText& OutFailReason)
{
	FString Template;
	if ( !ReadTemplateFile(TEXT("GameModule.h.template"), Template, OutFailReason) )
	{
		return false;
	}

	FString FinalOutput = Template.Replace(TEXT("%COPYRIGHT_LINE%"), *MakeCopyrightLine(), ESearchCase::CaseSensitive);
	FinalOutput = FinalOutput.Replace(TEXT("%PUBLIC_HEADER_INCLUDES%"), *MakeIncludeList(PublicHeaderIncludes), ESearchCase::CaseSensitive);

	return WriteOutputFile(NewBuildFileName, FinalOutput, OutFailReason);
}

bool GameProjectUtils::GeneratePluginModuleCPPFile(const FString& CPPFileName, const FString& ModuleName, const FString& StartupSourceCode, FText& OutFailReason)
{
	FString Template;
	if (!ReadTemplateFile(TEXT("PluginModule.cpp.template"), Template, OutFailReason))
	{
		return false;
	}

	FString FinalOutput = Template.Replace(TEXT("%COPYRIGHT_LINE%"), *MakeCopyrightLine(), ESearchCase::CaseSensitive);
	FinalOutput = FinalOutput.Replace(TEXT("%MODULE_NAME%"), *ModuleName, ESearchCase::CaseSensitive);
	FinalOutput = FinalOutput.Replace(TEXT("%MODULE_STARTUP_CODE%"), *StartupSourceCode, ESearchCase::CaseSensitive);

	return WriteOutputFile(CPPFileName, FinalOutput, OutFailReason);
}

bool GameProjectUtils::GeneratePluginModuleHeaderFile(const FString& HeaderFileName, const TArray<FString>& PublicHeaderIncludes, FText& OutFailReason)
{
	FString Template;
	if (!ReadTemplateFile(TEXT("PluginModule.h.template"), Template, OutFailReason))
	{
		return false;
	}

	FString FinalOutput = Template.Replace(TEXT("%COPYRIGHT_LINE%"), *MakeCopyrightLine(), ESearchCase::CaseSensitive);
	FinalOutput = FinalOutput.Replace(TEXT("%PUBLIC_HEADER_INCLUDES%"), *MakeIncludeList(PublicHeaderIncludes), ESearchCase::CaseSensitive);

	return WriteOutputFile(HeaderFileName, FinalOutput, OutFailReason);
}

FString GameProjectUtils::ReplaceWildcard(const FString& Input, const FString& From, const FString& To, bool bLeadingTab, bool bTrailingNewLine)
{
	FString Result = Input;
	FString WildCard = bLeadingTab ? TEXT("\t") : TEXT("");

	WildCard.Append(From);

	if (bTrailingNewLine)
	{
		WildCard.Append(LINE_TERMINATOR);
	}

	int32 NumReplacements = Result.ReplaceInline(*WildCard, *To, ESearchCase::CaseSensitive);

	// if replacement fails, try again using just the plain wildcard without tab and/or new line
	if (NumReplacements == 0)
	{
		Result = Result.Replace(*From, *To, ESearchCase::CaseSensitive);
	}

	return Result;
}

void GameProjectUtils::OnUpdateProjectConfirm()
{
	UpdateProject();
}

void GameProjectUtils::UpdateProject(const FProjectDescriptorModifier& Modifier)
{
	UpdateProject_Impl(&Modifier);
}

void GameProjectUtils::UpdateProject()
{
	UpdateProject_Impl(nullptr);
}

void GameProjectUtils::UpdateProject_Impl(const FProjectDescriptorModifier* Modifier)
{
	const FString& ProjectFilename = FPaths::GetProjectFilePath();
	const FString& ShortFilename = FPaths::GetCleanFilename(ProjectFilename);
	FText FailReason;
	FText UpdateMessage;
	SNotificationItem::ECompletionState NewCompletionState;
	if (UpdateGameProjectFile_Impl(ProjectFilename, FDesktopPlatformModule::Get()->GetCurrentEngineIdentifier(), Modifier, FailReason))
	{
		// Refresh the current in-memory project information from the new file on-disk.
		IProjectManager::Get().LoadProjectFile(ProjectFilename);

		// The project was updated successfully.
		FFormatNamedArguments Args;
		Args.Add( TEXT("ShortFilename"), FText::FromString( ShortFilename ) );
		UpdateMessage = FText::Format( LOCTEXT("ProjectFileUpdateComplete", "{ShortFilename} was successfully updated."), Args );
		NewCompletionState = SNotificationItem::CS_Success;
	}
	else
	{
		// The user chose to update, but the update failed. Notify the user.
		FFormatNamedArguments Args;
		Args.Add( TEXT("ShortFilename"), FText::FromString( ShortFilename ) );
		Args.Add( TEXT("FailReason"), FailReason );
		UpdateMessage = FText::Format( LOCTEXT("ProjectFileUpdateFailed", "{ShortFilename} failed to update. {FailReason}"), Args );
		NewCompletionState = SNotificationItem::CS_Fail;
	}

	if ( UpdateGameProjectNotification.IsValid() )
	{
		UpdateGameProjectNotification.Pin()->SetCompletionState(NewCompletionState);
		UpdateGameProjectNotification.Pin()->SetText(UpdateMessage);
		UpdateGameProjectNotification.Pin()->ExpireAndFadeout();
		UpdateGameProjectNotification.Reset();
	}
}

void GameProjectUtils::UpdateProject(const TArray<FString>* StartupModuleNames)
{
	UpdateProject(
		FProjectDescriptorModifier::CreateLambda(
			[StartupModuleNames](FProjectDescriptor& Desc)
			{
				if (StartupModuleNames != nullptr)
				{
					return UpdateStartupModuleNames(Desc, StartupModuleNames);
				}

				return false;
			}));
}

void GameProjectUtils::OnUpdateProjectCancel()
{
	if ( UpdateGameProjectNotification.IsValid() )
	{
		UpdateGameProjectNotification.Pin()->SetCompletionState(SNotificationItem::CS_None);
		UpdateGameProjectNotification.Pin()->ExpireAndFadeout();
		UpdateGameProjectNotification.Reset();
	}
}

void GameProjectUtils::TryMakeProjectFileWriteable(const FString& ProjectFile)
{
	if (IProjectManager::Get().IsSuppressingProjectFileWrite())
	{
		return;
	}

	// First attempt to check out the file if SCC is enabled
	if ( ISourceControlModule::Get().IsEnabled() )
	{
		FText FailReason;
		GameProjectUtils::CheckoutGameProjectFile(ProjectFile, FailReason);
	}

	// Check if it's writable
	if(FPlatformFileManager::Get().GetPlatformFile().IsReadOnly(*ProjectFile))
	{
		FText ShouldMakeProjectWriteable = LOCTEXT("ShouldMakeProjectWriteable_Message", "'{ProjectFilename}' is read-only and cannot be updated. Would you like to make it writeable?");

		FFormatNamedArguments Arguments;
		Arguments.Add( TEXT("ProjectFilename"), FText::FromString(ProjectFile));

		if(FMessageDialog::Open(EAppMsgType::YesNo, FText::Format(ShouldMakeProjectWriteable, Arguments)) == EAppReturnType::Yes)
		{
			FPlatformFileManager::Get().GetPlatformFile().SetReadOnly(*ProjectFile, false);
		}
	}
}

bool GameProjectUtils::UpdateGameProjectFile(const FString& ProjectFile, const FString& EngineIdentifier, const FProjectDescriptorModifier& Modifier, FText& OutFailReason)
{
	return UpdateGameProjectFile_Impl(ProjectFile, EngineIdentifier, &Modifier, OutFailReason);
}

bool GameProjectUtils::UpdateGameProjectFile(const FString& ProjectFile, const FString& EngineIdentifier, FText& OutFailReason)
{
	return UpdateGameProjectFile_Impl(ProjectFile, EngineIdentifier, nullptr, OutFailReason);
}

bool GameProjectUtils::UpdateGameProjectFile_Impl(const FString& ProjectFile, const FString& EngineIdentifier, const FProjectDescriptorModifier* Modifier, FText& OutFailReason)
{
	// Make sure we can write to the project file
	TryMakeProjectFileWriteable(ProjectFile);

	// Load the descriptor
	FProjectDescriptor Descriptor;
	if(Descriptor.Load(ProjectFile, OutFailReason))
	{
		if (Modifier && Modifier->IsBound() && !Modifier->Execute(Descriptor))
		{
			// If modifier returns false it means that we want to drop changes.
			return true;
		}

		// Update file on disk
		return Descriptor.Save(ProjectFile, OutFailReason) && FDesktopPlatformModule::Get()->SetEngineIdentifierForProject(ProjectFile, EngineIdentifier);
	}
	return false;
}

bool GameProjectUtils::UpdateGameProjectFile(const FString& ProjectFilename, const FString& EngineIdentifier, const TArray<FString>* StartupModuleNames, FText& OutFailReason)
{
	return UpdateGameProjectFile(ProjectFilename, EngineIdentifier,
		FProjectDescriptorModifier::CreateLambda(
			[StartupModuleNames](FProjectDescriptor& Desc)
			{
				if (StartupModuleNames != nullptr)
				{
					return UpdateStartupModuleNames(Desc, StartupModuleNames);
				}

				return false;
			}
		), OutFailReason);
}

bool GameProjectUtils::CheckoutGameProjectFile(const FString& ProjectFilename, FText& OutFailReason)
{
	if ( !ensure(ProjectFilename.Len()) )
	{
		OutFailReason = LOCTEXT("NoProjectFilename", "The project filename was not specified.");
		return false;
	}

	if ( !ISourceControlModule::Get().IsEnabled() )
	{
		OutFailReason = LOCTEXT("SCCDisabled", "Revision control is not enabled. Enable revision control in the preferences menu.");
		return false;
	}

	FString AbsoluteFilename = FPaths::ConvertRelativePathToFull(ProjectFilename);
	ISourceControlProvider& SourceControlProvider = ISourceControlModule::Get().GetProvider();
	FSourceControlStatePtr SourceControlState = SourceControlProvider.GetState(AbsoluteFilename, EStateCacheUsage::ForceUpdate);
	TArray<FString> FilesToBeCheckedOut;
	FilesToBeCheckedOut.Add(AbsoluteFilename);

	bool bSuccessfullyCheckedOut = false;
	OutFailReason = LOCTEXT("SCCStateInvalid", "Could not determine revision control state.");

	if(SourceControlState.IsValid())
	{
		if(SourceControlState->IsCheckedOut() || SourceControlState->IsAdded() || !SourceControlState->IsSourceControlled())
		{
			// Already checked out or opened for add... or not in the depot at all
			bSuccessfullyCheckedOut = true;
		}
		else if(SourceControlState->CanCheckout() || SourceControlState->IsCheckedOutOther())
		{
			bSuccessfullyCheckedOut = (SourceControlProvider.Execute(ISourceControlOperation::Create<FCheckOut>(), FilesToBeCheckedOut) == ECommandResult::Succeeded);
			if (!bSuccessfullyCheckedOut)
			{
				OutFailReason = LOCTEXT("SCCCheckoutFailed", "Failed to check out the project file.");
			}
		}
		else if(!SourceControlState->IsCurrent())
		{
			OutFailReason = LOCTEXT("SCCNotCurrent", "The project file is not at head revision.");
		}
	}

	return bSuccessfullyCheckedOut;
}

FString GameProjectUtils::GetDefaultProjectTemplateFilename()
{
	return TEXT("");
}

void FindCodeFiles(const TCHAR* BaseDirectory, TFunctionRef<bool(const TCHAR* FileName)> SourceFoundThreadSafeCallback)
{
	// Can't use IterateDirectoryRecursive here as we rely on some directory filtering without aborting whole enumeration
	struct FDirectoryVisitor : public IPlatformFile::FDirectoryVisitor
	{
		TFunctionRef<bool(const TCHAR* FileName)> SourceFoundThreadSafeCallback;
		FRWLock          DirectoriesLock;
		TArray<FString>& Directories;

		FDirectoryVisitor(TFunctionRef<bool(const TCHAR* FileName)> InSourceFoundThreadSafeCallback, TArray<FString>& InDirectories)
			: IPlatformFile::FDirectoryVisitor(EDirectoryVisitorFlags::ThreadSafe)
			, SourceFoundThreadSafeCallback(InSourceFoundThreadSafeCallback)
			, Directories(InDirectories)
		{
		}

		virtual bool Visit(const TCHAR* FilenameOrDirectory, bool bIsDirectory) override
		{
			if (bIsDirectory)
			{
				FString CleanDirectoryName(FPaths::GetCleanFilename(FilenameOrDirectory));
				// Skip directories like (i.e. .git) when finding code files to improve performance.
				if (!CleanDirectoryName.StartsWith(TEXT(".")))
				{
					FString Directory(FilenameOrDirectory);
					FRWScopeLock ScopeLock(DirectoriesLock, SLT_Write);
					Directories.Emplace(MoveTemp(Directory));
				}
			}
			else
			{
				FStringView FileName(FilenameOrDirectory);
				if (FileName.EndsWith(TEXT(".h")) || FileName.EndsWith(TEXT(".cpp")))
				{
					return SourceFoundThreadSafeCallback(FilenameOrDirectory);
				}
			}
			return true;
		}
	};

	TArray<FString> DirectoriesToVisitNext;
	DirectoriesToVisitNext.Add(BaseDirectory);
	
	TAtomic<bool> bResult(true);
	FDirectoryVisitor Visitor(SourceFoundThreadSafeCallback, DirectoriesToVisitNext);
	IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
	while (bResult && DirectoriesToVisitNext.Num() > 0)
	{
		TArray<FString> DirectoriesToVisit = MoveTemp(DirectoriesToVisitNext);
		ParallelFor(
			DirectoriesToVisit.Num(),
			[&PlatformFile , &DirectoriesToVisit, &Visitor, &bResult](int32 Index)
			{
				if (bResult && !PlatformFile.IterateDirectory(*DirectoriesToVisit[Index], Visitor))
				{
					bResult = false;
				}
			},
			EParallelForFlags::Unbalanced
		);
	}
}

void FindCodeFiles(const TCHAR* BaseDirectory, TArray<FString>& FileNames, int32 MaxNumFileNames)
{
	FRWLock FileNamesLock;
	auto SourceFoundThreadSafeCallback =
		[&FileNamesLock, &FileNames, MaxNumFileNames](const TCHAR* FileName) -> bool
		{
			// Prepare the string out of the lock
			FString SourceFile(FileName);

			FRWScopeLock ScopeLock(FileNamesLock, SLT_Write);
			// Make sure we don't go over the requested MaxNumFileNames even
			// when multiple threads are reaching this at the same time
			if (FileNames.Num() < MaxNumFileNames)
			{
				FileNames.Emplace(MoveTemp(SourceFile));
			}
			return FileNames.Num() < MaxNumFileNames;
		};

	FindCodeFiles(BaseDirectory, SourceFoundThreadSafeCallback);
}

void GameProjectUtils::GetProjectCodeFilenames(TArray<FString>& OutProjectCodeFilenames)
{
	FindCodeFiles(*FPaths::GameSourceDir(), OutProjectCodeFilenames, INT_MAX);
}

int32 GameProjectUtils::GetProjectCodeFileCount()
{
	TArray<FString> Filenames;
	GetProjectCodeFilenames(Filenames);
	return Filenames.Num();
}

void GameProjectUtils::GetProjectSourceDirectoryInfo(int32& OutNumCodeFiles, int64& OutDirectorySize)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(GameProjectUtils::GetProjectSourceDirectoryInfo)

	TAtomic<int32> NumCodeFiles(0);
	TAtomic<int64> DirectorySize(0);
	auto SourceFoundThreadSafeCallback =
		[&DirectorySize, &NumCodeFiles](const TCHAR* FileName) -> bool
		{
			NumCodeFiles++;
			DirectorySize += IFileManager::Get().FileSize(FileName);
			return true;
		};

	FindCodeFiles(*FPaths::GameSourceDir(), SourceFoundThreadSafeCallback);
	OutDirectorySize = DirectorySize;
	OutNumCodeFiles = NumCodeFiles;
}

bool GameProjectUtils::ProjectHasCodeFiles()
{
	TArray<FString> FileNames;
	FindCodeFiles(*FPaths::GameSourceDir(), FileNames, 1);
	return FileNames.Num() > 0;
}

TArray<FString> GameProjectUtils::GetRequiredAdditionalDependencies(const FNewClassInfo& ClassInfo)
{
	TArray<FString> Out;

	switch (ClassInfo.ClassType)
	{
	case FNewClassInfo::EClassType::SlateWidget:
	case FNewClassInfo::EClassType::SlateWidgetStyle:
		Out.Reserve(2);
		Out.Add(TEXT("Slate"));
		Out.Add(TEXT("SlateCore"));
		break;

	case FNewClassInfo::EClassType::UObject:
		auto ClassPackageName = ClassInfo.BaseClass->GetOutermost()->GetFName().ToString();

		checkf(ClassPackageName.StartsWith(TEXT("/Script/")), TEXT("Class outermost should start with /Script/"));

		Out.Add(ClassPackageName.Mid(8)); // Skip the /Script/ prefix.
		break;
	}

	return Out;
}

GameProjectUtils::EAddCodeToProjectResult GameProjectUtils::AddCodeToProject_Internal(const FString& NewClassName, const FString& NewClassPath, const FModuleContextInfo& ModuleInfo, const FNewClassInfo ParentClassInfo, const TSet<FString>& DisallowedHeaderNames, FString& OutHeaderFilePath, FString& OutCppFilePath, FText& OutFailReason, EReloadStatus& OutReloadStatus)
{
	if ( !ParentClassInfo.IsSet() )
	{
		OutFailReason = LOCTEXT("MissingParentClass", "You must specify a parent class");
		return EAddCodeToProjectResult::InvalidInput;
	}

	const FString CleanClassName = ParentClassInfo.GetCleanClassName(NewClassName);
	const FString FinalClassName = ParentClassInfo.GetFinalClassName(NewClassName);

	if (!IsValidClassNameForCreation(FinalClassName, ModuleInfo, DisallowedHeaderNames, OutFailReason))
	{
		return EAddCodeToProjectResult::InvalidInput;
	}

	if ( !FApp::HasProjectName() )
	{
		OutFailReason = LOCTEXT("AddCodeToProject_NoGameName", "You can not add code because you have not loaded a project.");
		return EAddCodeToProjectResult::FailedToAddCode;
	}

	FString NewHeaderPath;
	FString NewCppPath;
	if ( !CalculateSourcePaths(NewClassPath, ModuleInfo, NewHeaderPath, NewCppPath, &OutFailReason) )
	{
		return EAddCodeToProjectResult::FailedToAddCode;
	}

	FScopedSlowTask SlowTask( 7, LOCTEXT( "AddingCodeToProject", "Adding code to project..." ) );
	SlowTask.MakeDialog();

	SlowTask.EnterProgressFrame();

	auto RequiredDependencies = GetRequiredAdditionalDependencies(ParentClassInfo);
	RequiredDependencies.Remove(ModuleInfo.ModuleName);

	// Update project file if needed.
	auto bUpdateProjectModules = false;

	// If the project does not already contain code, add the primary game module
	TArray<FString> CreatedFiles;
	TArray<FString> StartupModuleNames;

	const bool bProjectHadCodeFiles = ProjectHasCodeFiles();
	if (!bProjectHadCodeFiles)
	{
		// Delete any generated intermediate code files. This ensures that blueprint projects with custom build settings can be converted to code projects without causing errors.
		IFileManager::Get().DeleteDirectory(*(FPaths::ProjectIntermediateDir() / TEXT("Source")), false, true);

		// We always add the basic source code to the root directory, not the potential sub-directory provided by NewClassPath
		const FString SourceDir = FPaths::GameSourceDir().LeftChop(1); // Trim the trailing /

		// Assuming the game name is the same as the primary game module name
		const FString GameModuleName = FApp::GetProjectName();

		if ( GenerateBasicSourceCode(SourceDir, GameModuleName, FPaths::ProjectDir(), StartupModuleNames, CreatedFiles, OutFailReason) )
		{
			bUpdateProjectModules = true;
		}
		else
		{
			DeleteCreatedFiles(SourceDir, CreatedFiles);
			return EAddCodeToProjectResult::FailedToAddCode;
		}
	}

	if (RequiredDependencies.Num() > 0 || bUpdateProjectModules)
	{
		UpdateProject(
			FProjectDescriptorModifier::CreateLambda(
			[&StartupModuleNames, &RequiredDependencies, &ModuleInfo, bUpdateProjectModules](FProjectDescriptor& Descriptor)
			{
				bool bNeedsUpdate = false;

				bNeedsUpdate |= UpdateStartupModuleNames(Descriptor, bUpdateProjectModules ? &StartupModuleNames : nullptr);
				bNeedsUpdate |= UpdateRequiredAdditionalDependencies(Descriptor, RequiredDependencies, ModuleInfo.ModuleName);

				return bNeedsUpdate;
			}));
	}

	SlowTask.EnterProgressFrame();

	// Class Header File
	const FString NewHeaderFilename = NewHeaderPath / ParentClassInfo.GetHeaderFilename(NewClassName);
	{
		FString UnusedSyncLocation;
		TArray<FString> ClassSpecifiers;

		// Set UCLASS() specifiers based on parent class type. Currently, only UInterface uses this.
		if (ParentClassInfo.ClassType == FNewClassInfo::EClassType::UInterface)
		{
			ClassSpecifiers.Add(TEXT("MinimalAPI"));
		}

		if ( GenerateClassHeaderFile(NewHeaderFilename, CleanClassName, ParentClassInfo, ClassSpecifiers, TEXT(""), TEXT(""), UnusedSyncLocation, ModuleInfo, false, OutFailReason) )
		{
			CreatedFiles.Add(NewHeaderFilename);
		}
		else
		{
			DeleteCreatedFiles(NewHeaderPath, CreatedFiles);
			return EAddCodeToProjectResult::FailedToAddCode;
		}
	}

	SlowTask.EnterProgressFrame();

	// Class CPP file
	const FString NewCppFilename = NewCppPath / ParentClassInfo.GetSourceFilename(NewClassName);
	{
		FString UnusedSyncLocation;
		if ( GenerateClassCPPFile(NewCppFilename, CleanClassName, ParentClassInfo, TArray<FString>(), TArray<FString>(), TEXT(""), UnusedSyncLocation, ModuleInfo, OutFailReason) )
		{
			CreatedFiles.Add(NewCppFilename);
		}
		else
		{
			DeleteCreatedFiles(NewCppPath, CreatedFiles);
			return EAddCodeToProjectResult::FailedToAddCode;
		}
	}

	SlowTask.EnterProgressFrame();

	TArray<FString> CreatedFilesForExternalAppRead;
	CreatedFilesForExternalAppRead.Reserve(CreatedFiles.Num());
	for (const FString& CreatedFile : CreatedFiles)
	{
		CreatedFilesForExternalAppRead.Add( IFileManager::Get().ConvertToAbsolutePathForExternalAppForRead(*CreatedFile) );
	}

	bool bGenerateProjectFiles = true;

	// First see if we can avoid a full generation by adding the new files to an already open project
	if ( bProjectHadCodeFiles && FSourceCodeNavigation::AddSourceFiles(CreatedFilesForExternalAppRead) )
	{
		// We managed the gather, so we can skip running the full generate
		bGenerateProjectFiles = false;
	}

	if ( bGenerateProjectFiles )
	{
		// Generate project files if we happen to be using a project file.
		if ( !FDesktopPlatformModule::Get()->GenerateProjectFiles(FPaths::RootDir(), FPaths::GetProjectFilePath(), GWarn) )
		{
			OutFailReason = LOCTEXT("FailedToGenerateProjectFiles", "Failed to generate project files.");
			return EAddCodeToProjectResult::FailedToHotReload;
		}
	}

	SlowTask.EnterProgressFrame();

	// Mark the files for add in SCC
	ISourceControlProvider& SourceControlProvider = ISourceControlModule::Get().GetProvider();
	if ( ISourceControlModule::Get().IsEnabled() && SourceControlProvider.IsAvailable() )
	{
		SourceControlProvider.Execute(ISourceControlOperation::Create<FMarkForAdd>(), CreatedFilesForExternalAppRead);
	}

	SlowTask.EnterProgressFrame( 1.0f, LOCTEXT("CompilingCPlusPlusCode", "Compiling new C++ code.  Please wait..."));

	OutHeaderFilePath = NewHeaderFilename;
	OutCppFilePath = NewCppFilename;
	OutReloadStatus = EReloadStatus::NotReloaded;

#if WITH_LIVE_CODING
	ILiveCodingModule* LiveCoding = FModuleManager::GetModulePtr<ILiveCodingModule>(LIVE_CODING_MODULE_NAME);
	if (LiveCoding != nullptr && LiveCoding->IsEnabledForSession())
	{
		if (!bProjectHadCodeFiles)
		{
			FMessageDialog::Open(EAppMsgType::Ok, LOCTEXT("LiveCodingNoSources", "Project now includes sources, please close the editor and build from your IDE."));
			return EAddCodeToProjectResult::Succeeded;
		}

		if (LiveCoding->AutomaticallyCompileNewClasses())
		{
			LiveCoding->Compile(ELiveCodingCompileFlags::None, nullptr);
			OutReloadStatus = EReloadStatus::Reloaded;
		}
		return EAddCodeToProjectResult::Succeeded;
	}
#endif

	if (!bProjectHadCodeFiles)
	{
		// This is the first time we add code to this project so compile its game DLL
		const FString GameModuleName = FApp::GetProjectName();
		check(ModuleInfo.ModuleName == GameModuleName);

		// Because this project previously didn't have any code, the UBT target name will just be UnrealEditor. Now that we've
		// added some code, the target name will be changed to match the editor target for the new source. 
		FString NewUBTTargetName = GameModuleName + TEXT("Editor");
		FPlatformMisc::SetUBTTargetName(*NewUBTTargetName);

		IHotReloadInterface& HotReloadSupport = FModuleManager::LoadModuleChecked<IHotReloadInterface>("HotReload");
		if (!HotReloadSupport.RecompileModule(*GameModuleName, *GWarn, ERecompileModuleFlags::ReloadAfterRecompile | ERecompileModuleFlags::ForceCodeProject))
		{
			OutFailReason = LOCTEXT("FailedToCompileNewGameModule", "Failed to compile newly created game module.");
			return EAddCodeToProjectResult::FailedToHotReload;
		}

		// Notify that we've created a brand new module
		FSourceCodeNavigation::AccessOnNewModuleAdded().Broadcast(*GameModuleName);
		OutReloadStatus = EReloadStatus::Reloaded;
	}
	else if (GetDefault<UEditorPerProjectUserSettings>()->bAutomaticallyHotReloadNewClasses)
	{
		FModuleStatus ModuleStatus;
		const FName ModuleFName = *ModuleInfo.ModuleName;
		if (ensure(FModuleManager::Get().QueryModule(ModuleFName, ModuleStatus)))
		{
			// Compile the module that the class was added to so that the newly added class with appear in the Content Browser
			TArray<UPackage*> PackagesToRebind;
			if (ModuleStatus.bIsLoaded)
			{
				const bool bIsHotReloadable = FModuleManager::Get().DoesLoadedModuleHaveUObjects(ModuleFName);
				if (bIsHotReloadable)
				{
					// Is there a UPackage with the same name as this module?
					const FString PotentialPackageName = FString(TEXT("/Script/")) + ModuleInfo.ModuleName;
					UPackage* Package = FindPackage(nullptr, *PotentialPackageName);
					if (Package)
					{
						PackagesToRebind.Add(Package);
					}
				}
			}

			IHotReloadInterface& HotReloadSupport = FModuleManager::LoadModuleChecked<IHotReloadInterface>("HotReload");
			if (PackagesToRebind.Num() > 0)
			{
				// Perform a hot reload
				ECompilationResult::Type CompilationResult = HotReloadSupport.RebindPackages( PackagesToRebind, EHotReloadFlags::WaitForCompletion, *GWarn );
				if( CompilationResult != ECompilationResult::Succeeded && CompilationResult != ECompilationResult::UpToDate )
				{
					OutFailReason = FText::Format(LOCTEXT("FailedToHotReloadModuleFmt", "Failed to automatically hot reload the '{0}' module."), FText::FromString(ModuleInfo.ModuleName));
					return EAddCodeToProjectResult::FailedToHotReload;
				}
			}
			else
			{
				// Perform a regular unload, then reload
				if (!HotReloadSupport.RecompileModule(ModuleFName, *GWarn, ERecompileModuleFlags::ReloadAfterRecompile))
				{
					OutFailReason = FText::Format(LOCTEXT("FailedToCompileModuleFmt", "Failed to automatically compile the '{0}' module."), FText::FromString(ModuleInfo.ModuleName));
					return EAddCodeToProjectResult::FailedToHotReload;
				}
			}
		}
		OutReloadStatus = EReloadStatus::Reloaded;
	}

	return EAddCodeToProjectResult::Succeeded;
}

bool GameProjectUtils::FindSourceFileInProject(const FString& InFilename, const FString& InSearchPath, FString& OutPath)
{
	TArray<FString> Filenames;
	IFileManager::Get().FindFilesRecursive(Filenames, *InSearchPath, *InFilename, true, false, false);

	if(Filenames.Num())
	{
		// Assume it's the first match (we should really only find a single file with a given name within a project anyway)
		OutPath = Filenames[0];
		return true;
	}

	return false;
}


void GameProjectUtils::HarvestCursorSyncLocation( FString& FinalOutput, FString& OutSyncLocation )
{
	OutSyncLocation.Empty();

	// Determine the cursor focus location if this file will by synced after creation
	TArray<FString> Lines;
	FinalOutput.ParseIntoArray( Lines, TEXT( "\n" ), false );
	for( int32 LineIdx = 0; LineIdx < Lines.Num(); ++LineIdx )
	{
		const FString& Line = Lines[ LineIdx ];
		int32 CharLoc = Line.Find( TEXT( "%CURSORFOCUSLOCATION%" ) );
		if( CharLoc != INDEX_NONE )
		{
			// Found the sync marker
			OutSyncLocation = FString::Printf( TEXT( "%d:%d" ), LineIdx + 1, CharLoc + 1 );
			break;
		}
	}

	// If we did not find the sync location, just sync to the top of the file
	if( OutSyncLocation.IsEmpty() )
	{
		OutSyncLocation = TEXT( "1:1" );
	}

	// Now remove the cursor focus marker
	FinalOutput = FinalOutput.Replace(TEXT("%CURSORFOCUSLOCATION%"), TEXT(""), ESearchCase::CaseSensitive);
}

bool GameProjectUtils::InsertFeaturePacksIntoINIFile(const FProjectInformation& InProjectInfo, FText& OutFailReason)
{
	OutFailReason = LOCTEXT("DeprecatedStarterPack", "Deprecated: Ability to add Starter Content has been removed");
	return false;
}

bool GameProjectUtils::AddSharedContentToProject(const FProjectInformation &InProjectInfo, TArray<FString> &CreatedFiles, FText& OutFailReason)
{
	//const FString TemplateName = FPaths::GetBaseFilename(InProjectInfo.TemplateFile);
	const FString SrcFolder = FPaths::GetPath(InProjectInfo.TemplateFile);
	const FString DestFolder = FPaths::GetPath(InProjectInfo.ProjectFilename);

	// Now any packs specified in the template def.
	UTemplateProjectDefs* TemplateDefs = LoadTemplateDefs(SrcFolder);
	if (TemplateDefs != NULL)
	{
		EFeaturePackDetailLevel RequiredDetail = EFeaturePackDetailLevel::High;
		if (InProjectInfo.TargetedHardware == EHardwareClass::Mobile)
		{
			RequiredDetail = EFeaturePackDetailLevel::Standard;
		}

		TUniquePtr<FFeaturePackContentSource> TempFeaturePack = MakeUnique<FFeaturePackContentSource>();
		bool bCopied = TempFeaturePack->InsertAdditionalResources(TemplateDefs->SharedContentPacks, RequiredDetail, DestFolder, CreatedFiles);
		if( bCopied == false )
		{
			FFormatNamedArguments Args;
			Args.Add(TEXT("TemplateName"), FText::FromString(SrcFolder));
			OutFailReason = FText::Format(LOCTEXT("SharedResourceError", "Error adding shared resources for '{TemplateName}'."), Args);
			return false;
		}

		if (!TemplateDefs->Variants.IsEmpty() && InProjectInfo.Variant != NAME_None)
		{
			const FTemplateVariant* Variant = TemplateDefs->FindVariant(InProjectInfo.Variant);
			if (!Variant)
			{
				FFormatNamedArguments Args;
				Args.Add(TEXT("TemplateName"), FText::FromString(SrcFolder));
				OutFailReason = FText::Format(LOCTEXT("UnknownVariantError", "Unknown variant specified for '{TemplateName}'."), Args);
				return false;
			}
			
			TUniquePtr<FFeaturePackContentSource> TempVariantFeaturePack = MakeUnique<FFeaturePackContentSource>();
			bool bVariantCopied = TempVariantFeaturePack->InsertAdditionalResources(Variant->SharedContentPacks, RequiredDetail, DestFolder, CreatedFiles);
			if( bVariantCopied == false )
			{
				FFormatNamedArguments Args;
				Args.Add(TEXT("VariantName"), Variant->GetDisplayNameText());
				Args.Add(TEXT("TemplateName"), FText::FromString(SrcFolder));
				OutFailReason = FText::Format(LOCTEXT("VariantSharedResourceError", "Error adding variant '{VariantName}' shared resources for '{TemplateName}'."), Args);
				return false;
			}
		}
	}
	return true;
}

#undef LOCTEXT_NAMESPACE
