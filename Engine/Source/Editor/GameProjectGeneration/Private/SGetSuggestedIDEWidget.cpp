// Copyright Epic Games, Inc. All Rights Reserved.


#include "SGetSuggestedIDEWidget.h"

#include "Delegates/Delegate.h"
#include "EngineAnalytics.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Notifications/NotificationManager.h"
#include "HAL/Platform.h"
#include "HAL/PlatformProcess.h"
#include "IAnalyticsProviderET.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Layout/Children.h"
#include "SPrimaryButton.h"
#include "SourceCodeNavigation.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Widgets/SWindow.h"

class SWidget;

#define LOCTEXT_NAMESPACE "GameProjectGeneration"

void SGetDisableIDEWidget::Construct(const FArguments& InArgs)
{
	ChildSlot
	[
		CreateGetDisableIDEWidget()
	];
}

TSharedRef<SWidget> SGetDisableIDEWidget::CreateGetDisableIDEWidget()
{
	return
		SNew(SPrimaryButton)
		.Text(FText::Format(LOCTEXT("IDEDisableButton", "Disable {0}"), FSourceCodeNavigation::GetSuggestedSourceCodeIDE()))
		.OnClicked(this, &SGetDisableIDEWidget::OnDisableIDEClicked);
}

FReply SGetDisableIDEWidget::OnDisableIDEClicked()
{
	FSourceCodeNavigation::SetPreferredAccessor(TEXT("NullSourceCodeAccessor"));

	return FReply::Handled();
}

TSharedPtr<SNotificationItem> SGetSuggestedIDEWidget::IDEDownloadNotification;

void SGetSuggestedIDEWidget::Construct(const FArguments& InArgs)
{
	SetVisibility(InArgs._VisibilityOverride.IsSet() ? InArgs._VisibilityOverride : TAttribute<EVisibility>(this, &SGetSuggestedIDEWidget::GetVisibility));

	ChildSlot
	[
		CreateGetSuggestedIDEWidget()
	];
}

TSharedRef<SWidget> SGetSuggestedIDEWidget::CreateGetSuggestedIDEWidget()
{
	if (FSourceCodeNavigation::GetCanDirectlyInstallSourceCodeIDE())
	{
		// If the installer for this platform's IDE can be downloaded and launched directly, show a button
		return			
			SNew(SPrimaryButton)
			.Text(FText::Format(LOCTEXT("IDEInstallButtonText", "Install {0}"), FSourceCodeNavigation::GetSuggestedSourceCodeIDE()))
			.OnClicked(this, &SGetSuggestedIDEWidget::OnInstallIDEClicked);
	}
	else
	{
		return	
			SNew(SPrimaryButton)
			.Text(FText::Format(LOCTEXT("IDEDownloadLinkText", "Download {0}"), FSourceCodeNavigation::GetSuggestedSourceCodeIDE()))
			.OnClicked(this, &SGetSuggestedIDEWidget::OnDownloadIDEClicked, FSourceCodeNavigation::GetSuggestedSourceCodeIDEDownloadURL());
	}
}

EVisibility SGetSuggestedIDEWidget::GetVisibility() const
{
	return FSourceCodeNavigation::IsCompilerAvailable() ? EVisibility::Collapsed : EVisibility::Visible;
}

FReply SGetSuggestedIDEWidget::OnDownloadIDEClicked(FString URL)
{
	FPlatformProcess::LaunchURL(*URL, NULL, NULL);

	return FReply::Handled();
}

FReply SGetSuggestedIDEWidget::OnInstallIDEClicked()
{
	// If the notification faded out, allow it to be deleted.
	if (IDEDownloadNotification.IsValid() && IDEDownloadNotification->GetCompletionState() == SNotificationItem::CS_None)
	{
		IDEDownloadNotification.Reset();
	}

	// If we have a notification already for this task and its corresponding task hasn't yet completed, do nothing.
	if (!IDEDownloadNotification.IsValid() || IDEDownloadNotification->GetCompletionState() != SNotificationItem::ECompletionState::CS_Pending)
	{
		FText MessageText = FText::Format(LOCTEXT("DownloadingIDEInstaller", "Downloading {0} Installer..."), FSourceCodeNavigation::GetSuggestedSourceCodeIDE(true));

		if (!IDEDownloadNotification.IsValid())
		{
			FNotificationInfo Info(MessageText);
			Info.bUseLargeFont = false;
			Info.bFireAndForget = false;
			Info.bUseSuccessFailIcons = true;
			Info.bUseThrobber = true;

			IDEDownloadNotification = FSlateNotificationManager::Get().AddNotification(Info);
		}
		else
		{
			// Just reuse the same notification, since it hasn't faded offscreen yet.
			IDEDownloadNotification->SetText(MessageText);
		}
		IDEDownloadNotification->SetCompletionState(SNotificationItem::ECompletionState::CS_Pending);

		FSourceCodeNavigation::DownloadAndInstallSuggestedIDE(FOnIDEInstallerDownloadComplete::CreateStatic(&SGetSuggestedIDEWidget::OnIDEInstallerDownloadComplete));

		if (FEngineAnalytics::IsAvailable())
		{
			FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.InstalledIDE"));
		}
	}


	TSharedPtr<SWindow> ThisWindow = FSlateApplication::Get().FindWidgetWindow(AsShared());

	if (ThisWindow.IsValid() && ThisWindow->IsModalWindow())
	{
		// If this window is modal, close it to unblock the IDE request and allow it to finish...as long as another
		// modal window isn't opened on top of it
		ThisWindow->RequestDestroyWindow();
	}

	return FReply::Handled();
}

void SGetSuggestedIDEWidget::OnIDEInstallerDownloadComplete(bool bWasSuccessful)
{
	if (IDEDownloadNotification.IsValid())
	{
		if (bWasSuccessful)
		{
			IDEDownloadNotification->SetCompletionState(SNotificationItem::ECompletionState::CS_Success);
			IDEDownloadNotification->SetText(LOCTEXT("IDEDownloadSuccess", "Starting installation..."));
		}
		else
		{
			IDEDownloadNotification->SetCompletionState(SNotificationItem::ECompletionState::CS_Fail);
			IDEDownloadNotification->SetText(LOCTEXT("IDEDownloadFailure", "Failed to download installer. Please check your internet connection."));
		}

		IDEDownloadNotification->ExpireAndFadeout();
		IDEDownloadNotification = nullptr;
	}
}

#undef LOCTEXT_NAMESPACE
