// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "Input/Reply.h"
#include "Layout/Visibility.h"
#include "Misc/Attribute.h"
#include "Templates/SharedPointer.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class SWidget;

/**
 * Either a button to directly install or a hyperlink to a website to download the suggested IDE for the platform.
 * Only visible when no compiler is available.
 */
class SGetSuggestedIDEWidget : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SGetSuggestedIDEWidget)
	{}
		SLATE_ATTRIBUTE(EVisibility, VisibilityOverride)
	SLATE_END_ARGS()

	/** Constructs this widget with InArgs */
	void Construct(const FArguments& InArgs);
	
private:
	/** Creates the appropriate widget to display for the platform */
	TSharedRef<SWidget> CreateGetSuggestedIDEWidget();

	/** Gets the visibility of the global error label IDE Link */
	EVisibility GetVisibility() const;

	/** Handler for when the error label IDE hyperlink was clicked */
	FReply OnDownloadIDEClicked(FString URL);

	/** Handler for when the install IDE button was clicked */
	FReply OnInstallIDEClicked();

	/** Handler for when the suggested IDE installer has finished downloading */
	static void OnIDEInstallerDownloadComplete(bool bWasSuccessful);

private:

	/** Handle to the notification displayed when downloading an IDE installer */
	static TSharedPtr<class SNotificationItem> IDEDownloadNotification;

};

/**
 * A button to disable requiring an IDE to be install 
 * Only visible on platforms that dont require an IDE to compile code and when no IDE is available
 */
class SGetDisableIDEWidget : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SGetDisableIDEWidget) {}
	SLATE_END_ARGS()

	/** Constructs this widget with InArgs */
	void Construct(const FArguments& InArgs);
	
private:
	/** Creates the appropriate widget to display for the platform */
	TSharedRef<SWidget> CreateGetDisableIDEWidget();

	/** Handler for when the disable button is clicked */
	FReply OnDisableIDEClicked();
};
