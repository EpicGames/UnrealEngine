// Copyright Epic Games, Inc. All Rights Reserved.


#include "SNewClassDialog.h"
#include "Misc/MessageDialog.h"
#include "HAL/FileManager.h"
#include "Misc/App.h"
#include "SlateOptMacros.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SGridPanel.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Framework/Docking/TabManager.h"
#include "Styling/AppStyle.h"
#include "Engine/Blueprint.h"
#include "Editor/EditorPerProjectUserSettings.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "Interfaces/IProjectManager.h"
#include "SGetSuggestedIDEWidget.h"
#include "SourceCodeNavigation.h"
#include "ClassViewerModule.h"
#include "ClassViewerFilter.h"
#include "IContentBrowserSingleton.h"
#include "ContentBrowserModule.h"
#include "SClassViewer.h"
#include "DesktopPlatformModule.h"
#include "IDocumentation.h"
#include "EditorClassUtils.h"
#include "UObject/UObjectHash.h"
#include "Widgets/Workflow/SWizard.h"
#include "Widgets/Input/SHyperlink.h"
#include "TutorialMetaData.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "IAssetTools.h"
#include "AssetRegistry/AssetRegistryModule.h"

#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "FeaturedClasses.inl"
#include "Subsystems/AssetEditorSubsystem.h"
#include "Editor.h"
#include "Styling/StyleColors.h"
#include "Widgets/Input/SSegmentedControl.h"
#include "ClassIconFinder.h"
#include "SWarningOrErrorBox.h"
#include "Dialog/DialogCommands.h"

#define LOCTEXT_NAMESPACE "GameProjectGeneration"

/** The last selected module name. Meant to keep the same module selected after first selection */
FString SNewClassDialog::LastSelectedModuleName;


struct FParentClassItem
{
	FNewClassInfo ParentClassInfo;

	FParentClassItem(const FNewClassInfo& InParentClassInfo)
		: ParentClassInfo(InParentClassInfo)
	{}
};

class FNativeClassParentFilter : public IClassViewerFilter
{
public:
	FNativeClassParentFilter()
	{
		ProjectModules = GameProjectUtils::GetCurrentProjectModules();
	}

	virtual bool IsClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const UClass* InClass, TSharedRef< FClassViewerFilterFuncs > InFilterFuncs ) override
	{
		// We allow a class that belongs to any module in the current project, as you don't actually choose the destination module until after you've selected your parent class
		return GameProjectUtils::IsValidBaseClassForCreation(InClass, ProjectModules);
	}

	virtual bool IsUnloadedClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const TSharedRef< const IUnloadedBlueprintData > InUnloadedClassData, TSharedRef< FClassViewerFilterFuncs > InFilterFuncs) override
	{
		return false;
	}

private:
	/** The list of currently available modules for this project */
	TArray<FModuleContextInfo> ProjectModules;
};

static void FindPublicEngineHeaderFiles(TArray<FString>& OutFiles, const FString& Path)
{
	TArray<FString> ModuleDirs;
	IFileManager::Get().FindFiles(ModuleDirs, *(Path / TEXT("*")), false, true);
	for (const FString& ModuleDir : ModuleDirs)
	{
		IFileManager::Get().FindFilesRecursive(OutFiles, *(Path / ModuleDir / TEXT("Classes")), TEXT("*.h"), true, false, false);
		IFileManager::Get().FindFilesRecursive(OutFiles, *(Path / ModuleDir / TEXT("Public")), TEXT("*.h"), true, false, false);
	}
}

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void SNewClassDialog::Construct( const FArguments& InArgs )
{
	ClassDomain = InArgs._ClassDomain;
	bSyncContentBrowserToNewClass = InArgs._SyncContentBrowserToNewClass;

	{
		TArray<FModuleContextInfo> CurrentModules = GameProjectUtils::GetCurrentProjectModules();
		check(CurrentModules.Num()); // this should never happen since GetCurrentProjectModules is supposed to add a dummy runtime module if the project currently has no modules

		TArray<FModuleContextInfo> CurrentPluginModules = GameProjectUtils::GetCurrentProjectPluginModules();

		CurrentModules.Append(CurrentPluginModules);

		AvailableModules.Reserve(CurrentModules.Num());
		for(const FModuleContextInfo& ModuleInfo : CurrentModules)
		{
			AvailableModules.Emplace(MakeShareable(new FModuleContextInfo(ModuleInfo)));
		}

		Algo::SortBy(AvailableModules, &FModuleContextInfo::ModuleName);
	}

	// If we've been given an initial path that maps to a valid project module, use that as our initial module and path

	if (ClassDomain == EClassDomain::Blueprint)
	{
		NewClassPath = InArgs._InitialPath.IsEmpty() ? TEXT("/Game") : InArgs._InitialPath;

		// Pick a valid default path if the path is not writable
		FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
		NewClassPath = ContentBrowserModule.Get().GetInitialPathToSaveAsset(FContentBrowserItemPath(NewClassPath, EContentBrowserPathType::Internal)).GetInternalPathString();
	}
	else if(!InArgs._InitialPath.IsEmpty())
	{
		const FString AbsoluteInitialPath = FPaths::ConvertRelativePathToFull(InArgs._InitialPath);
		for(const auto& AvailableModule : AvailableModules)
		{
			if(AbsoluteInitialPath.StartsWith(AvailableModule->ModuleSourcePath))
			{
				SelectedModuleInfo = AvailableModule;
				NewClassPath = AbsoluteInitialPath;
				break;
			}
		}
	}

	DefaultClassPrefix = InArgs._DefaultClassPrefix;
	DefaultClassName = InArgs._DefaultClassName;

	// If we didn't get given a valid path override (see above), try and automatically work out the best default module
	// If we have a runtime module with the same name as our project, then use that
	// Otherwise, set out default target module as the first runtime module in the list
	if(ClassDomain == EClassDomain::Native && !SelectedModuleInfo.IsValid())
	{
		const FString ProjectName = FApp::GetProjectName();

		// Find initially selected module based on simple fallback in this order..
		// Previously selected module, main project module, the first runtime module
		TSharedPtr<FModuleContextInfo> ProjectModule;
		TSharedPtr<FModuleContextInfo> RuntimeModule;

		for (const auto& AvailableModule : AvailableModules)
		{
			// Check if this module matches our last used
			if (AvailableModule->ModuleName == LastSelectedModuleName)
			{
				SelectedModuleInfo = AvailableModule;
				break;
			}

			if (AvailableModule->ModuleName == ProjectName)
			{
				ProjectModule = AvailableModule;
			}
			
			//Check if this is a runtime module and if we haven't already found one
			if (AvailableModule->ModuleType == EHostType::Runtime && RuntimeModule == nullptr)
			{
				RuntimeModule = AvailableModule;
			}
		}

		if (!SelectedModuleInfo.IsValid())
		{
			if (ProjectModule.IsValid())
			{
				// use the project module we found
				SelectedModuleInfo = ProjectModule;
			}
			else if (RuntimeModule.IsValid())
			{
				// use the first runtime module we found
				SelectedModuleInfo = RuntimeModule;
			}
			else
			{
				// default to just the first module
				SelectedModuleInfo = AvailableModules[0];
			}
		}

		NewClassPath = SelectedModuleInfo->ModuleSourcePath;
	}

	ClassLocation = GameProjectUtils::EClassLocation::UserDefined; // the first call to UpdateInputValidity will set this correctly based on NewClassPath

	ParentClassInfo = FNewClassInfo(InArgs._Class);

	bShowFullClassTree = false;

	LastPeriodicValidityCheckTime = 0;
	PeriodicValidityCheckFrequency = 4;
	bLastInputValidityCheckSuccessful = true;
	bPreventPeriodicValidityChecksUntilNextChange = false;

	FClassViewerInitializationOptions Options;
	Options.Mode = EClassViewerMode::ClassPicker;
	Options.DisplayMode = EClassViewerDisplayMode::TreeView;
	Options.bIsActorsOnly = false;
	Options.bIsPlaceableOnly = false;
	Options.bIsBlueprintBaseOnly = false;
	Options.bShowUnloadedBlueprints = false;
	Options.bShowNoneOption = false;
	Options.bShowObjectRootClass = true;
	Options.bExpandRootNodes = true;

	TSharedPtr<IClassViewerFilter> ClassFilter = InArgs._ClassViewerFilter;
	if (!ClassFilter.IsValid() && InArgs._ClassDomain == EClassDomain::Native)
	{
		// Prevent creating native classes based on blueprint classes
		ClassFilter = MakeShared<FNativeClassParentFilter>();
	}

	if (ClassFilter.IsValid())
	{
		Options.ClassFilters.Add(ClassFilter.ToSharedRef());

		// Only show the Object root class if it's a valid base (this helps keep the tree clean)
		if (!ClassFilter->IsClassAllowed(Options, UObject::StaticClass(), MakeShared<FClassViewerFilterFuncs>()))
		{
			Options.bShowObjectRootClass = false;
		}
	}

	ClassViewer = StaticCastSharedRef<SClassViewer>(FModuleManager::LoadModuleChecked<FClassViewerModule>("ClassViewer").CreateClassViewer(Options, FOnClassPicked::CreateSP(this, &SNewClassDialog::OnAdvancedClassSelected)));

	// Make sure the featured classes all pass the active class filter
	TArray<FNewClassInfo> ValidatedFeaturedClasses;
	ValidatedFeaturedClasses.Reserve(InArgs._FeaturedClasses.Num());
	for (const FNewClassInfo& FeaturedClassInfo : InArgs._FeaturedClasses)
	{
		if (FeaturedClassInfo.ClassType != FNewClassInfo::EClassType::UObject || ClassViewer->IsClassAllowed(FeaturedClassInfo.BaseClass))
		{
			ValidatedFeaturedClasses.Add(FeaturedClassInfo);
		}
	}

	SetupParentClassItems(ValidatedFeaturedClasses);
	UpdateInputValidity();

	TSharedRef<SWidget> DocWidget = IDocumentation::Get()->CreateAnchor(TAttribute<FString>(this, &SNewClassDialog::GetSelectedParentDocLink));
	DocWidget->SetVisibility(TAttribute<EVisibility>(this, &SNewClassDialog::GetDocLinkVisibility));

	const float EditableTextHeight = 26.0f;

	IContentBrowserSingleton& ContentBrowser = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser").Get();

	FPathPickerConfig BlueprintPathConfig;
	if (ClassDomain == EClassDomain::Blueprint)
	{
		BlueprintPathConfig.DefaultPath = NewClassPath;
		BlueprintPathConfig.bFocusSearchBoxWhenOpened = false;
		BlueprintPathConfig.bAllowContextMenu = true;
		BlueprintPathConfig.bAllowClassesFolder = false;
		BlueprintPathConfig.bAllowReadOnlyFolders = false;
		BlueprintPathConfig.OnPathSelected = FOnPathSelected::CreateSP(this, &SNewClassDialog::OnBlueprintPathSelected);
		BlueprintPathConfig.bNotifyDefaultPathSelected = true;
	}

	OnAddedToProject = InArgs._OnAddedToProject;

	ChildSlot
	[
		SNew(SBorder)
		.Padding(18.0f)
		.BorderImage( FAppStyle::GetBrush("Docking.Tab.ContentAreaBrush") )
		[
			SNew(SVerticalBox)
			.AddMetaData<FTutorialMetaData>(TEXT("AddCodeMajorAnchor"))

			+SVerticalBox::Slot()
			[
				SAssignNew( MainWizard, SWizard)
				.ShowPageList(false)
				.CanFinish(this, &SNewClassDialog::CanFinish)
				.FinishButtonText( ClassDomain == EClassDomain::Native ? LOCTEXT("FinishButtonText_Native", "Create Class") : FText::Format(LOCTEXT("FinishButtonText_Blueprint", "Create {0} Class"), ParentClassInfo.IsSet() ? ParentClassInfo.GetClassName() : FText::FromStringView(TEXT("Blueprint"))))
				.FinishButtonToolTip (
					ClassDomain == EClassDomain::Native ?
					LOCTEXT("FinishButtonToolTip_Native", "Creates the code files to add your new class.") : 
					FText::Format(LOCTEXT("FinishButtonToolTip_Blueprint", "Creates the new class based on the specified parent {0} class."), ParentClassInfo.IsSet() ? ParentClassInfo.GetClassName() : FText::FromStringView(TEXT("Blueprint")))
					)
				.OnCanceled(this, &SNewClassDialog::CancelClicked)
				.OnFinished(this, &SNewClassDialog::FinishClicked)
				.ShowHistoryButtons(ClassDomain == EClassDomain::Native || !ParentClassInfo.IsSet())
				.InitialPageIndex(ParentClassInfo.IsSet() ? 1 : 0)
				.PageFooter()
				[
					// Get IDE information
					SNew(SBorder)
					.Visibility( this, &SNewClassDialog::GetGlobalErrorLabelVisibility )
					.BorderImage(FAppStyle::Get().GetBrush("RoundedError"))
					.Padding(FMargin(0.0f, 5.0f))
					.Content()
					[
						SNew(SHorizontalBox)

						+SHorizontalBox::Slot()
						.VAlign(VAlign_Center)
						.Padding(5.f, 2.f)
						.AutoWidth()
						[
							SNew(SImage)
							.Image(FAppStyle::Get().GetBrush("Icons.ErrorWithColor"))
						]

						+SHorizontalBox::Slot()
						.VAlign(VAlign_Center)
						[
							SNew(STextBlock)
							.Text( this, &SNewClassDialog::GetGlobalErrorLabelText )
							.AutoWrapText(true)
						]

						+SHorizontalBox::Slot()
						.VAlign(VAlign_Center)
						.HAlign(HAlign_Center)
						.AutoWidth()
						.Padding(5.f, 0.f)
						[
							SNew(SGetSuggestedIDEWidget)
						]
					]
				]
				
				// Choose parent class
				+SWizard::Page()
				.CanShow(!ParentClassInfo.IsSet()) // We can't move to this widget page if we've been given a parent class to use
				[
					SNew(SVerticalBox)

					// Title
					+SVerticalBox::Slot()
					.AutoHeight()
					.Padding(0.0f)
					[
						SNew(STextBlock)
						.Font(FAppStyle::Get().GetFontStyle("HeadingExtraSmall"))
						.Text( LOCTEXT( "ParentClassTitle", "Choose Parent Class" ) )
						.TransformPolicy(ETextTransformPolicy::ToUpper)
					]

					+ SVerticalBox::Slot()
					.AutoHeight()
					.HAlign(HAlign_Center)
					[
						SNew(SSegmentedControl<bool>)
						.OnValueChanged(this, &SNewClassDialog::OnFullClassTreeChanged)
						.Value(this, &SNewClassDialog::IsFullClassTreeShown)
						+SSegmentedControl<bool>::Slot(false)
						.Text(LOCTEXT("CommonClasses", "Common Classes"))
						+ SSegmentedControl<bool>::Slot(true)
						.Text(LOCTEXT("AllClasses", "All Classes"))
					]

					// Page description and view options
					+SVerticalBox::Slot()
					.AutoHeight()
					.Padding(0.0f, 10.0f)
					[
						SNew(STextBlock)
						.Text(
							ClassDomain == EClassDomain::Native ?
							LOCTEXT("ChooseParentClassDescription_Native", "This will add a C++ header and source code file to your game project.") :
							FText::Format(LOCTEXT("ChooseParentClassDescription_Blueprint", "This will add a new class inheriting from {0} to your game project."), ParentClassInfo.IsSet() ? ParentClassInfo.GetClassName() : FText::FromStringView(TEXT("Blueprint")))
						)
					]

					// Add Code list
					+SVerticalBox::Slot()
					.FillHeight(1.f)
					.Padding(0.0f, 10.0f)
					[
						SNew(SBorder)
						.AddMetaData<FTutorialMetaData>(TEXT("AddCodeOptions"))
						.BorderImage( FAppStyle::GetBrush("ToolPanel.GroupBorder") )
						[
							SNew(SVerticalBox)

							+SVerticalBox::Slot()
							[
								// Basic view
								SAssignNew(ParentClassListView, SListView< TSharedPtr<FParentClassItem> >)
								.ListItemsSource(&ParentClassItemsSource)
								.SelectionMode(ESelectionMode::Single)
								.ClearSelectionOnClick(false)
								.OnGenerateRow(this, &SNewClassDialog::MakeParentClassListViewWidget)
								.OnMouseButtonDoubleClick( this, &SNewClassDialog::OnParentClassItemDoubleClicked )
								.OnSelectionChanged(this, &SNewClassDialog::OnClassSelected)
								.Visibility(this, &SNewClassDialog::GetBasicParentClassVisibility)
							]

							+SVerticalBox::Slot()
							[
								// Advanced view
								SNew(SBox)
								.Visibility(this, &SNewClassDialog::GetAdvancedParentClassVisibility)
								[
									ClassViewer.ToSharedRef()
								]
							]
						]
					]

					// Class selection
					+SVerticalBox::Slot()
					.Padding(30.0f, 2.0f)
					.AutoHeight()
					[
						SNew(SGridPanel)
						.FillColumn(1, 1.0f)

						// Class label
						+ SGridPanel::Slot(0,0)
						.VAlign(VAlign_Center)
						.Padding(2.0f, 2.0f, 10.0f, 2.0f)
						.HAlign(HAlign_Left)
						[
							SNew(STextBlock)
							.Text(LOCTEXT("ParentClassLabel", "Selected Class"))
						]

						+ SGridPanel::Slot(0, 1)
						.VAlign(VAlign_Center)
						.Padding(2.0f, 2.0f, 10.0f, 2.0f)
						.HAlign(HAlign_Left)
						[
							SNew(STextBlock)
							.Visibility(ClassDomain == EClassDomain::Blueprint ? EVisibility::Collapsed : EVisibility::Visible)
							.Text(LOCTEXT("ParentClassSourceLabel", "Selected Class Source"))
						]

						+ SGridPanel::Slot(1, 0)
						.VAlign(VAlign_Center)
						.Padding(2.0f)
						.HAlign(HAlign_Left)
						[
							SNew(SHorizontalBox)

							+ SHorizontalBox::Slot()
							.VAlign(VAlign_Center)
							.AutoWidth()
							[
								SNew(STextBlock)
								.Text(this, &SNewClassDialog::GetSelectedParentClassName)
							]

							+ SHorizontalBox::Slot()
							.VAlign(VAlign_Center)
							.AutoWidth()
							[
								DocWidget
							]
						]

						+ SGridPanel::Slot(1, 1)
						.VAlign(VAlign_Center)
						.Padding(2.0f)
						.HAlign(HAlign_Left)
						[
							SNew(SHyperlink)
							.Style(FAppStyle::Get(), "Common.GotoNativeCodeHyperlink")
							.OnNavigate(this, &SNewClassDialog::OnEditCodeClicked)
							.Text(this, &SNewClassDialog::GetSelectedParentClassFilename)
							.ToolTipText(FText::Format(LOCTEXT("GoToCode_ToolTip", "Click to open this source file in {0}"), FSourceCodeNavigation::GetSelectedSourceCodeIDE()))
							.Visibility(this, &SNewClassDialog::GetSourceHyperlinkVisibility)
						]
					]
				]

				// Name class
				+SWizard::Page()
				.OnEnter(this, &SNewClassDialog::OnNamePageEntered)
				[
					SNew(SScrollBox)
					.AllowContentToShrink(true)

					// Title
					+SScrollBox::Slot()
					.AutoSize()
					.Padding(0.0f)
					[
						SNew(STextBlock)
						.Font(FAppStyle::Get().GetFontStyle("HeadingExtraSmall"))
						.Text( this, &SNewClassDialog::GetNameClassTitle )
						.TransformPolicy(ETextTransformPolicy::ToUpper)
					]

					+SScrollBox::Slot()
					.FillSize(1.0f)
					.MinSize(358.0f)
					.Padding(0.0f, 10.0f)
					[
						SNew(SVerticalBox)

						+SVerticalBox::Slot()
						.AutoHeight()
						.Padding(0.0f, 0.0f, 0.0f, 5.0f)
						[
							SNew(STextBlock)
							.Text(LOCTEXT("ClassNameDescription", "Enter a name for your new class. Class names may only contain alphanumeric characters, and may not contain a space.") )
								.AutoWrapText(true)
						]

						+SVerticalBox::Slot()
						.AutoHeight()
						.Padding(0.0f, 0.0f, 0.0f, 2.0f)
						[
							SNew(STextBlock)
							.Text( ClassDomain == EClassDomain::Native ?
								LOCTEXT("ClassNameDetails_Native", "When you click the \"Create\" button below, a header (.h) file and a source (.cpp) file will be made using this name.") :
								FText::Format(LOCTEXT("ClassNameDetails_Blueprint", "When you click the \"Create\" button below, a new class inheriting from {0} will be created."), ParentClassInfo.IsSet() ? ParentClassInfo.GetClassName() : FText::FromStringView(TEXT("Blueprint")))
								)
								.AutoWrapText(true)
						]

						// Name Error label
						+SVerticalBox::Slot()
						.AutoHeight()
						.Padding(0.0f, 5.0f)
						[
							SNew(SWarningOrErrorBox)
							.MessageStyle(EMessageStyle::Error)
							.Visibility(this, &SNewClassDialog::GetNameErrorLabelVisibility)
							.Message(this, &SNewClassDialog::GetNameErrorLabelText)
						]

						// Properties
						+SVerticalBox::Slot()
						.FillHeight(1.0f)
						[
							SNew(SBorder)
							.BorderImage(FAppStyle::GetBrush("DetailsView.CategoryTop"))
							.BorderBackgroundColor(FLinearColor(0.6f, 0.6f, 0.6f, 1.0f ))
							.Padding(FMargin(6.0f, 4.0f, 7.0f, 4.0f))
							[
								SNew(SGridPanel)
								.FillColumn(1, 1.0f)
								.FillRow(2, 1.0f)

								// Class type label
								+ SGridPanel::Slot(0, 0)
								.VAlign(VAlign_Center)
								.Padding(0.0f, 0.0f, 12.0f, 0.0f)
								[
									SNew(STextBlock)
									.Visibility(ClassDomain == EClassDomain::Blueprint ? EVisibility::Collapsed : EVisibility::Visible)
									.Text(LOCTEXT("ClassTypeLabel", "Class Type"))
								]

								+SGridPanel::Slot(1,0)
								.VAlign(VAlign_Center)
								.HAlign(HAlign_Left)
								.Padding(2.0f)
								[
									SNew(SSegmentedControl<GameProjectUtils::EClassLocation>)
									.Visibility(ClassDomain == EClassDomain::Blueprint ? EVisibility::Collapsed : EVisibility::Visible)
									.OnValueChanged(this, &SNewClassDialog::OnClassLocationChanged)
									.Value(this, &SNewClassDialog::IsClassLocationActive)
									+ SSegmentedControl<GameProjectUtils::EClassLocation>::Slot(GameProjectUtils::EClassLocation::Public)
									.Text(LOCTEXT("Public", "Public"))
									.ToolTip(LOCTEXT("ClassLocation_Public", "A public class can be included and used inside other modules in addition to the module it resides in"))
									+ SSegmentedControl<GameProjectUtils::EClassLocation>::Slot(GameProjectUtils::EClassLocation::Private)
									.Text(LOCTEXT("Private", "Private"))
									.ToolTip(LOCTEXT("ClassLocation_Private", "A private class can only be included and used within the module it resides in"))
								]
								// Name label
								+SGridPanel::Slot(0, 1)
								.VAlign(VAlign_Center)
								.Padding(0.0f, 0.0f, 12.0f, 0.0f)
								[
									SNew(STextBlock)
									.Text( LOCTEXT( "NameLabel", "Name" ) )
								]

								// Name edit box
								+SGridPanel::Slot(1, 1)
								.Padding(0.0f, 3.0f)
								.VAlign(VAlign_Center)
								[
									SNew(SBox)
									.HeightOverride(EditableTextHeight)
									.AddMetaData<FTutorialMetaData>(TEXT("ClassName"))
									[
										SNew(SHorizontalBox)

										+SHorizontalBox::Slot()
										.FillWidth(.7f)
										[
											SAssignNew( ClassNameEditBox, SEditableTextBox)
											.Text( this, &SNewClassDialog::OnGetClassNameText )
											.OnTextChanged( this, &SNewClassDialog::OnClassNameTextChanged )
											.OnTextCommitted( this, &SNewClassDialog::OnClassNameTextCommitted )
										]

										+SHorizontalBox::Slot()
										.AutoWidth()
										.Padding(6.0f, 0.0f, 0.0f, 0.0f)
										[
											SAssignNew(AvailableModulesCombo, SComboBox<TSharedPtr<FModuleContextInfo>>)
											.Visibility(ClassDomain == EClassDomain::Blueprint ? EVisibility::Collapsed : EVisibility::Visible)
											.ToolTipText( LOCTEXT("ModuleComboToolTip", "Choose the target module for your new class") )
											.OptionsSource( &AvailableModules )
											.InitiallySelectedItem( SelectedModuleInfo )
											.OnSelectionChanged( this, &SNewClassDialog::SelectedModuleComboBoxSelectionChanged )
											.OnGenerateWidget( this, &SNewClassDialog::MakeWidgetForSelectedModuleCombo )
											[
												SNew(STextBlock)
												.Text( this, &SNewClassDialog::GetSelectedModuleComboText )
											]
										]
									]
								]

								// Path label
								+SGridPanel::Slot(0, 2)
								.VAlign(ClassDomain == EClassDomain::Blueprint ? VAlign_Top : VAlign_Center)
								.Padding(0.0f, 0.0f, 12.0f, 0.0f)
								[
									SNew(STextBlock)
									.Text( LOCTEXT( "PathLabel", "Path" ) )
								]

								// Path edit box
								+SGridPanel::Slot(1, 2)
								.Padding(0.0f, 3.0f)
								.VAlign(VAlign_Fill)
								[
									SNew(SVerticalBox)

									// Blueprint Class asset path
									+ SVerticalBox::Slot()
									.FillHeight(1.0f)
									.Padding(0.0f)
									[
										SNew(SBox)
										.Visibility(ClassDomain == EClassDomain::Blueprint ? EVisibility::Visible : EVisibility::Collapsed)
										[
											SNew(SVerticalBox)
											
											+SVerticalBox::Slot()
											.AutoHeight()
											[
												SNew(STextBlock)
												.Text(this, &SNewClassDialog::OnGetClassPathText)
											]

											+SVerticalBox::Slot()
											.FillHeight(1.0f)
											[
												ContentBrowser.CreatePathPicker(BlueprintPathConfig)
											]
										]
									]

									// Native C++ path
									+ SVerticalBox::Slot()
									.Padding(0.0f)
									.AutoHeight()
									[
										SNew(SBox)
										.Visibility(ClassDomain == EClassDomain::Blueprint ? EVisibility::Collapsed : EVisibility::Visible)
										.HeightOverride(EditableTextHeight)
										.AddMetaData<FTutorialMetaData>(TEXT("Path"))
										[
											SNew(SHorizontalBox)

											+SHorizontalBox::Slot()
											.FillWidth(1.0f)
											[
												SNew(SEditableTextBox)
												.Text(this, &SNewClassDialog::OnGetClassPathText)
												.OnTextChanged(this, &SNewClassDialog::OnClassPathTextChanged)
											]

											+SHorizontalBox::Slot()
											.AutoWidth()
											.Padding(6.0f, 1.0f, 0.0f, 0.0f)
											[
												SNew(SButton)
												.VAlign(VAlign_Center)
												.ButtonStyle(FAppStyle::Get(), "SimpleButton")
												.OnClicked(this, &SNewClassDialog::HandleChooseFolderButtonClicked)
												[
													SNew(SImage)
													.Image(FAppStyle::Get().GetBrush("Icons.FolderClosed"))
													.ColorAndOpacity(FSlateColor::UseForeground())
												]
											]
										]
									]
								]

								// Header output label
								+SGridPanel::Slot(0, 3)
								.VAlign(VAlign_Center)
								.Padding(0.0f, 0.0f, 12.0f, 0.0f)
								[
									SNew(STextBlock)
									.Visibility(ClassDomain == EClassDomain::Blueprint ? EVisibility::Collapsed : EVisibility::Visible)
									.Text( LOCTEXT( "HeaderFileLabel", "Header File" ) )
								]

								// Header output text
								+SGridPanel::Slot(1, 3)
								.Padding(0.0f, 3.0f)
								.VAlign(VAlign_Center)
								[
									SNew(SBox)
									.Visibility(ClassDomain == EClassDomain::Blueprint ? EVisibility::Collapsed : EVisibility::Visible)
									.VAlign(VAlign_Center)
									.HeightOverride(EditableTextHeight)
									[
										SNew(STextBlock)
										.Text(this, &SNewClassDialog::OnGetClassHeaderFileText)
									]
								]

								// Source output label
								+SGridPanel::Slot(0, 4)
								.VAlign(VAlign_Center)
								.Padding(0.0f, 0.0f, 12.0f, 0.0f)
								[
									SNew(STextBlock)
									.Visibility(ClassDomain == EClassDomain::Blueprint ? EVisibility::Collapsed : EVisibility::Visible)
									.Text( LOCTEXT( "SourceFileLabel", "Source File" ) )
								]

								// Source output text
								+SGridPanel::Slot(1, 4)
								.Padding(0.0f, 3.0f)
								.VAlign(VAlign_Center)
								[
									SNew(SBox)
									.Visibility(ClassDomain == EClassDomain::Blueprint ? EVisibility::Collapsed : EVisibility::Visible)
									.VAlign(VAlign_Center)
									.HeightOverride(EditableTextHeight)
									[
										SNew(STextBlock)
										.Text(this, &SNewClassDialog::OnGetClassSourceFileText)
									]
								]
							]
						]
					]
				]
			]
		]
	];

	// Select the first item
	if ( InArgs._Class == NULL && ParentClassItemsSource.Num() > 0 )
	{
		ParentClassListView->SetSelection(ParentClassItemsSource[0], ESelectInfo::Direct);
	}

	TSharedPtr<SWindow> ParentWindow = InArgs._ParentWindow;
	if (ParentWindow.IsValid())
	{
		ParentWindow.Get()->SetWidgetToFocusOnActivate(ParentClassListView);
	}

}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

TSharedPtr<SWidget> SNewClassDialog::GetWidgetToFocusOnActivate()
{
	return MainWizard->GetWidgetToFocusOnActivate();
}

void SNewClassDialog::Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime )
{
	// Every few seconds, the class name/path is checked for validity in case the disk contents changed and the location is now valid or invalid.
	// After class creation, periodic checks are disabled to prevent a brief message indicating that the class you created already exists.
	// This feature is re-enabled if the user did not restart and began editing parameters again.
	if ( !bPreventPeriodicValidityChecksUntilNextChange && (InCurrentTime > LastPeriodicValidityCheckTime + PeriodicValidityCheckFrequency) )
	{
		UpdateInputValidity();
	}
}

TSharedRef<ITableRow> SNewClassDialog::MakeParentClassListViewWidget(TSharedPtr<FParentClassItem> ParentClassItem, const TSharedRef<STableViewBase>& OwnerTable)
{
	if ( !ensure(ParentClassItem.IsValid()) )
	{
		return SNew( STableRow<TSharedPtr<FParentClassItem>>, OwnerTable );
	}

	if ( !ParentClassItem->ParentClassInfo.IsSet() )
	{
		return SNew( STableRow<TSharedPtr<FParentClassItem>>, OwnerTable );
	}

	const FText ClassName = ParentClassItem->ParentClassInfo.GetClassName();
	const FText ClassFullDescription = ParentClassItem->ParentClassInfo.GetClassDescription(/*bFullDescription*/true);
	const FText ClassShortDescription = ParentClassItem->ParentClassInfo.GetClassDescription(/*bFullDescription*/false);
	const UClass* Class = ParentClassItem->ParentClassInfo.BaseClass;
	const FSlateBrush* const ClassBrush = FClassIconFinder::FindThumbnailForClass(Class);

	const int32 ItemHeight = 64;
	return
		SNew( STableRow<TSharedPtr<FParentClassItem>>, OwnerTable )
		.Padding(4.0f)
		.Style(FAppStyle::Get(), "NewClassDialog.ParentClassListView.TableRow")
		.ToolTip(IDocumentation::Get()->CreateToolTip(ClassFullDescription, nullptr, FEditorClassUtils::GetDocumentationPage(Class), FEditorClassUtils::GetDocumentationExcerpt(Class)))
		[
			SNew(SBox)
			.HeightOverride(static_cast<float>(ItemHeight))
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Center)
				.Padding(8.0f)
				[
					SNew(SBox)
					.HeightOverride(ItemHeight / 2.0f)
					.WidthOverride(ItemHeight / 2.0f)
					[
						SNew(SImage)
						.Image(ClassBrush)
					]
				]

				+ SHorizontalBox::Slot()
				.FillWidth(1.0f)
				.VAlign(VAlign_Center)
				.Padding(4.0f)
				[
					SNew(SVerticalBox)
					+SVerticalBox::Slot()
					[
						SNew(STextBlock)
						.TextStyle(FAppStyle::Get(), "DialogButtonText")
						.Text(ClassName)
					]

					+SVerticalBox::Slot()
					[
						SNew(STextBlock)
						.Text(ClassShortDescription)
						.AutoWrapText(true)
					]
				]
			]
		];
}

FText SNewClassDialog::GetSelectedParentClassName() const
{
	return ParentClassInfo.IsSet() ? ParentClassInfo.GetClassName() : FText::GetEmpty();
}

FString GetClassHeaderPath(const UClass* Class)
{
	if (Class)
	{
		FString ClassHeaderPath;
		if (FSourceCodeNavigation::FindClassHeaderPath(Class, ClassHeaderPath) && IFileManager::Get().FileSize(*ClassHeaderPath) != INDEX_NONE)
		{
			return ClassHeaderPath;
		}
	}
	return FString();
}

EVisibility SNewClassDialog::GetSourceHyperlinkVisibility() const
{
	if (ClassDomain == EClassDomain::Blueprint)
	{
		return EVisibility::Collapsed;
	}

	return (ParentClassInfo.GetBaseClassHeaderFilename().Len() > 0 ? EVisibility::Visible : EVisibility::Hidden);
}

FText SNewClassDialog::GetSelectedParentClassFilename() const
{
	const FString ClassHeaderPath = ParentClassInfo.GetBaseClassHeaderFilename();
	if (ClassHeaderPath.Len() > 0)
	{
		return FText::FromString(FPaths::GetCleanFilename(*ClassHeaderPath));
	}
	return FText::GetEmpty();
}

EVisibility SNewClassDialog::GetDocLinkVisibility() const
{
	return (ParentClassInfo.BaseClass == nullptr || FEditorClassUtils::GetDocumentationLink(ParentClassInfo.BaseClass).IsEmpty() ? EVisibility::Hidden : EVisibility::Visible);
}

FString SNewClassDialog::GetSelectedParentDocLink() const
{
	return FEditorClassUtils::GetDocumentationLink(ParentClassInfo.BaseClass);
}

void SNewClassDialog::OnEditCodeClicked()
{
	const FString ClassHeaderPath = ParentClassInfo.GetBaseClassHeaderFilename();
	if (ClassHeaderPath.Len() > 0)
	{
		const FString AbsoluteHeaderPath = IFileManager::Get().ConvertToAbsolutePathForExternalAppForRead(*ClassHeaderPath);
		FSourceCodeNavigation::OpenSourceFileAsync(AbsoluteHeaderPath);
	}
}


void SNewClassDialog::OnParentClassItemDoubleClicked( TSharedPtr<FParentClassItem> TemplateItem )
{
	// Advance to the name page
	const int32 NamePageIdx = 1;
	if ( MainWizard->CanShowPage(NamePageIdx) )
	{
		MainWizard->ShowPage(NamePageIdx);
	}
}

void SNewClassDialog::OnClassSelected(TSharedPtr<FParentClassItem> Item, ESelectInfo::Type SelectInfo)
{
	if ( Item.IsValid() )
	{
		ClassViewer->ClearSelection();
		ParentClassInfo = Item->ParentClassInfo;
	}
	else
	{
		ParentClassInfo = FNewClassInfo();
	}
}

void SNewClassDialog::OnAdvancedClassSelected(UClass* Class)
{
	ParentClassListView->ClearSelection();
	ParentClassInfo = FNewClassInfo(Class);
}

bool SNewClassDialog::IsFullClassTreeShown() const
{
	return bShowFullClassTree;
}

void SNewClassDialog::OnFullClassTreeChanged(bool bInShowFullClassTree)
{
	bShowFullClassTree = bInShowFullClassTree;
}

EVisibility SNewClassDialog::GetBasicParentClassVisibility() const
{
	return bShowFullClassTree ? EVisibility::Collapsed : EVisibility::Visible;
}

EVisibility SNewClassDialog::GetAdvancedParentClassVisibility() const
{
	return bShowFullClassTree ? EVisibility::Visible : EVisibility::Collapsed;
}

EVisibility SNewClassDialog::GetNameErrorLabelVisibility() const
{
	return GetNameErrorLabelText().IsEmpty() ? EVisibility::Hidden : EVisibility::Visible;
}

FText SNewClassDialog::GetNameErrorLabelText() const
{
	if ( !bLastInputValidityCheckSuccessful )
	{
		return LastInputValidityErrorText;
	}

	return FText::GetEmpty();
}

EVisibility SNewClassDialog::GetGlobalErrorLabelVisibility() const
{
	return GetGlobalErrorLabelText().IsEmpty() ? EVisibility::Collapsed : EVisibility::Visible;
}

FText SNewClassDialog::GetGlobalErrorLabelText() const
{
	if ( ClassDomain == EClassDomain::Native && !FSourceCodeNavigation::IsCompilerAvailable() )
	{
#if PLATFORM_LINUX
		return FText::Format(LOCTEXT("NoCompilerFoundNewClassLinux", "Your IDE {0} is missing or incorrectly configured, please consider using {1}"),
			FSourceCodeNavigation::GetSelectedSourceCodeIDE(), FSourceCodeNavigation::GetSuggestedSourceCodeIDE());
#else
		return FText::Format(LOCTEXT("NoCompilerFoundNewClass", "No compiler was found. In order to use C++ code, you must first install {0}."), FSourceCodeNavigation::GetSuggestedSourceCodeIDE());
#endif
	}

	return FText::GetEmpty();
}

void SNewClassDialog::OnNamePageEntered()
{
	// Set the default class name based on the selected parent class, eg MyActor
	const FString ParentClassName = ParentClassInfo.GetClassNameCPP();
	const FString PotentialNewClassName = FString::Printf(TEXT("%s%s"), 
		DefaultClassPrefix.IsEmpty() ? TEXT("My") : *DefaultClassPrefix, 
		DefaultClassName.IsEmpty() ? (ParentClassName.IsEmpty() ? TEXT("Class") : *ParentClassName) : *DefaultClassName);

	// Only set the default if the user hasn't changed the class name from the previous default
	if(LastAutoGeneratedClassName.IsEmpty() || NewClassName == LastAutoGeneratedClassName)
	{
		NewClassName = PotentialNewClassName;
		LastAutoGeneratedClassName = PotentialNewClassName;
	}

	UpdateInputValidity();

	// Steal keyboard focus to accelerate name entering
	FSlateApplication::Get().SetKeyboardFocus(ClassNameEditBox, EFocusCause::SetDirectly);
}

FText SNewClassDialog::GetNameClassTitle() const
{
	static const FString NoneString = TEXT("None");

	const FText ParentClassName = GetSelectedParentClassName();
	if(!ParentClassName.IsEmpty() && ParentClassName.ToString() != NoneString)
	{
		return FText::Format( LOCTEXT( "NameClassTitle", "Name Your New {0}" ), ParentClassName );
	}

	return LOCTEXT( "NameClassGenericTitle", "Name Your New Class" );
}

FText SNewClassDialog::OnGetClassNameText() const
{
	return FText::FromString(NewClassName);
}

void SNewClassDialog::OnClassNameTextChanged(const FText& NewText)
{
	NewClassName = NewText.ToString();
	UpdateInputValidity();
}

void SNewClassDialog::OnClassNameTextCommitted(const FText& NewText, ETextCommit::Type CommitType)
{
	if (CommitType == ETextCommit::OnEnter)
	{
		if (CanFinish())
		{
			FinishClicked();
		}
	}
}

FText SNewClassDialog::OnGetClassPathText() const
{
	return FText::FromString(NewClassPath);
}

void SNewClassDialog::OnClassPathTextChanged(const FText& NewText)
{
	NewClassPath = NewText.ToString();

	// If the user has selected a path which matches the root of a known module, then update our selected module to be that module
	for(const auto& AvailableModule : AvailableModules)
	{
		if(NewClassPath.StartsWith(AvailableModule->ModuleSourcePath))
		{
			SelectedModuleInfo = AvailableModule;
			AvailableModulesCombo->SetSelectedItem(SelectedModuleInfo);
			break;
		}
	}

	UpdateInputValidity();
}

void SNewClassDialog::OnBlueprintPathSelected(const FString& NewPath)
{
	IsBlueprintPathSelected = true;
	NewClassPath = NewPath;
	UpdateInputValidity();
}

FText SNewClassDialog::OnGetClassHeaderFileText() const
{
	return FText::FromString(CalculatedClassHeaderName);
}

FText SNewClassDialog::OnGetClassSourceFileText() const
{
	return FText::FromString(CalculatedClassSourceName);
}

void SNewClassDialog::CancelClicked()
{
	CloseContainingWindow();
}

bool SNewClassDialog::CanFinish() const
{
	return bLastInputValidityCheckSuccessful && ParentClassInfo.IsSet() && (ClassDomain == EClassDomain::Blueprint || FSourceCodeNavigation::IsCompilerAvailable()) && (ClassDomain != EClassDomain::Blueprint || IsBlueprintPathSelected);
}

void SNewClassDialog::FinishClicked()
{
	check(CanFinish());

	if (ClassDomain == EClassDomain::Blueprint)
	{
		FString PackagePath = NewClassPath / NewClassName;

		if (!ParentClassInfo.BaseClass)
		{
			// @todo show fail reason in error label
			FMessageDialog::Open(EAppMsgType::Ok, FText::Format(LOCTEXT("AddCodeFailed_Blueprint_NoBase", "No parent class has been specified. Failed to generate new {0} class."), FText::FromString(NewClassName)));
		}
		else if (FindObject<UBlueprint>(nullptr, *PackagePath))
		{
			// @todo show fail reason in error label
			FMessageDialog::Open(EAppMsgType::Ok, FText::Format(LOCTEXT("AddCodeFailed_Blueprint_AlreadyExists", "The chosen class name ({0}) already exists, please try again with a different name."), FText::FromString(NewClassName)));
		}
		else if (!NewClassPath.IsEmpty() && !NewClassName.IsEmpty())
		{
			UPackage* Package = CreatePackage( *PackagePath);
			if (Package)
			{
				// Create and init a new Blueprint
				UBlueprint* NewBP = FKismetEditorUtilities::CreateBlueprint(const_cast<UClass*>(ParentClassInfo.BaseClass), Package, FName(*NewClassName), BPTYPE_Normal);
				if (NewBP)
				{
					// Set the default "AssetAccessSpecifier" state
					Package->SetAssetAccessSpecifier(IAssetTools::Get().ShouldCreateAssetsAsExternallyReferenceableForPath(PackagePath) ? EAssetAccessSpecifier::Public : EAssetAccessSpecifier::Private);

					// Notify the asset registry
					FAssetRegistryModule::AssetCreated(NewBP);

					// Mark the package dirty...
					Package->MarkPackageDirty();

					OnAddedToProject.ExecuteIfBound( NewClassName, PackagePath, FString() );

					if (bSyncContentBrowserToNewClass)
					{
						// Sync the content browser to the new asset
						GEditor->SyncBrowserToObject(NewBP);
					}

					// Open the editor for the new asset
					GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(NewBP);

					// Successfully created the code and potentially opened the IDE. Close the dialog.
					CloseContainingWindow();

					return;
				}
			}
		}

		// @todo show fail reason in error label
		// Failed to add blueprint
		const FText Message = FText::Format( LOCTEXT("AddCodeFailed_Blueprint", "Failed to create package for class {0}. Please try again with a different name."), FText::FromString(NewClassName) );
		FMessageDialog::Open(EAppMsgType::Ok, Message);
	}
	else
	{

		FString HeaderFilePath;
		FString CppFilePath;

		// Track the selected module name so we can default to this next time
		LastSelectedModuleName = SelectedModuleInfo->ModuleName;

		GameProjectUtils::EReloadStatus ReloadStatus;
		FText FailReason;
		const TSet<FString>& DisallowedHeaderNames = FSourceCodeNavigation::GetSourceFileDatabase().GetDisallowedHeaderNames();
		const GameProjectUtils::EAddCodeToProjectResult AddCodeResult = GameProjectUtils::AddCodeToProject(NewClassName, NewClassPath, *SelectedModuleInfo, ParentClassInfo, DisallowedHeaderNames, HeaderFilePath, CppFilePath, FailReason, ReloadStatus);
		if (AddCodeResult == GameProjectUtils::EAddCodeToProjectResult::Succeeded)
		{
			OnAddedToProject.ExecuteIfBound( NewClassName, NewClassPath, SelectedModuleInfo->ModuleName );

			// Reload current project to take into account any new state
			IProjectManager::Get().LoadProjectFile(FPaths::GetProjectFilePath());

			// Prevent periodic validity checks. This is to prevent a brief error message about the class already existing while you are exiting.
			bPreventPeriodicValidityChecksUntilNextChange = true;

			// Display a nag if we didn't automatically hot-reload for the newly added class
			bool bWasReloaded = ReloadStatus == GameProjectUtils::EReloadStatus::Reloaded;

			if( bWasReloaded )
			{
				FNotificationInfo Notification( FText::Format( LOCTEXT("AddedClassSuccessNotification", "Added new class {0}"), FText::FromString(NewClassName) ) );
				FSlateNotificationManager::Get().AddNotification( Notification );
			}

			if ( HeaderFilePath.IsEmpty() || CppFilePath.IsEmpty() || !FSlateApplication::Get().SupportsSourceAccess() )
			{
				if( !bWasReloaded )
				{
					// Code successfully added, notify the user. We are either running on a platform that does not support source access or a file was not given so don't ask about editing the file
					const FText Message = FText::Format( 
						LOCTEXT("AddCodeSuccessWithHotReload", "Successfully added class '{0}', however you must recompile the '{1}' module before it will appear in the Content Browser.")
						, FText::FromString(NewClassName), FText::FromString(SelectedModuleInfo->ModuleName) );
					FMessageDialog::Open(EAppMsgType::Ok, Message);
				}
				else
				{
					// Code was added and hot reloaded into the editor, but the user doesn't have a code IDE installed so we can't open the file to edit it now
				}
			}
			else
			{
				bool bEditSourceFilesNow = false;
				if( bWasReloaded )
				{
					// Code was hot reloaded, so always edit the new classes now
					bEditSourceFilesNow = true;
				}
				else
				{
					// Code successfully added, notify the user and ask about opening the IDE now
					const FText Message = FText::Format( 
						LOCTEXT("AddCodeSuccessWithHotReloadAndSync", "Successfully added class '{0}', however you must recompile the '{1}' module before it will appear in the Content Browser.\n\nWould you like to edit the code now?")
						, FText::FromString(NewClassName), FText::FromString(SelectedModuleInfo->ModuleName) );
					bEditSourceFilesNow = ( FMessageDialog::Open( EAppMsgType::YesNo, Message ) == EAppReturnType::Yes );
				}

				if( bEditSourceFilesNow )
				{
					TArray<FString> SourceFiles;
					SourceFiles.Add(IFileManager::Get().ConvertToAbsolutePathForExternalAppForRead(*HeaderFilePath));
					SourceFiles.Add(IFileManager::Get().ConvertToAbsolutePathForExternalAppForRead(*CppFilePath));

					FSourceCodeNavigation::OpenSourceFilesAsync(SourceFiles);
				}
			}

			// Sync the content browser to the new class
			UPackage* const ClassPackage = FindPackage(nullptr, *(FString("/Script/") + SelectedModuleInfo->ModuleName));
			if ( ClassPackage )
			{
				UClass* const NewClass = static_cast<UClass*>(FindObjectWithOuter(ClassPackage, UClass::StaticClass(), *NewClassName));
				if ( NewClass )
				{
					GEditor->SyncBrowserToObject(NewClass);
				}
			}

			// Successfully created the code and potentially opened the IDE. Close the dialog.
			CloseContainingWindow();
		}
		else if (AddCodeResult == GameProjectUtils::EAddCodeToProjectResult::FailedToHotReload)
		{
			OnAddedToProject.ExecuteIfBound( NewClassName, NewClassPath, SelectedModuleInfo->ModuleName );

			// Prevent periodic validity checks. This is to prevent a brief error message about the class already existing while you are exiting.
			bPreventPeriodicValidityChecksUntilNextChange = true;

			// Failed to compile new code
			const FText Message = FText::Format(
				LOCTEXT("AddCodeFailed_HotReloadFailed", "Successfully added class '{0}', however you must recompile the '{1}' module before it will appear in the Content Browser. {2}\n\nWould you like to open the Output Log to see more details?")
				, FText::FromString(NewClassName), FText::FromString(SelectedModuleInfo->ModuleName), FailReason );
			if( FMessageDialog::Open(EAppMsgType::YesNo, Message) == EAppReturnType::Yes )
			{
				FGlobalTabmanager::Get()->TryInvokeTab(FName("OutputLog"));
			}

			// We did manage to add the code itself, so we can close the dialog.
			CloseContainingWindow();
		}
		else
		{
			// @todo show fail reason in error label
			// Failed to add code
			const FText Message = FText::Format( LOCTEXT("AddCodeFailed_AddCodeFailed", "Failed to add class '{0}'. {1}"), FText::FromString(NewClassName), FailReason );
			FMessageDialog::Open(EAppMsgType::Ok, Message);
		}
	}
}

FReply SNewClassDialog::HandleChooseFolderButtonClicked()
{
	IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
	if ( DesktopPlatform )
	{
		TSharedPtr<SWindow> ParentWindow = FSlateApplication::Get().FindWidgetWindow(AsShared());
		void* ParentWindowWindowHandle = (ParentWindow.IsValid()) ? ParentWindow->GetNativeWindow()->GetOSWindowHandle() : nullptr;

		FString FolderName;
		const FString Title = LOCTEXT("NewClassBrowseTitle", "Choose a source location").ToString();
		const bool bFolderSelected = DesktopPlatform->OpenDirectoryDialog(
			ParentWindowWindowHandle,
			Title,
			NewClassPath,
			FolderName
			);

		if ( bFolderSelected )
		{
			if ( !FolderName.EndsWith(TEXT("/")) )
			{
				FolderName += TEXT("/");
			}

			NewClassPath = FolderName;

			// If the user has selected a path which matches the root of a known module, then update our selected module to be that module
			for(const auto& AvailableModule : AvailableModules)
			{
				if(NewClassPath.StartsWith(AvailableModule->ModuleSourcePath))
				{
					SelectedModuleInfo = AvailableModule;
					AvailableModulesCombo->SetSelectedItem(SelectedModuleInfo);
					break;
				}
			}

			UpdateInputValidity();
		}
	}

	return FReply::Handled();
}

FText SNewClassDialog::GetSelectedModuleComboText() const
{
	FFormatNamedArguments Args;
	Args.Add(TEXT("ModuleName"), FText::FromString(SelectedModuleInfo->ModuleName));
	Args.Add(TEXT("ModuleType"), FText::FromString(EHostType::ToString(SelectedModuleInfo->ModuleType)));
	return FText::Format(LOCTEXT("ModuleComboEntry", "{ModuleName} ({ModuleType})"), Args);
}

void SNewClassDialog::SelectedModuleComboBoxSelectionChanged(TSharedPtr<FModuleContextInfo> Value, ESelectInfo::Type SelectInfo)
{
	const FString& OldModulePath = SelectedModuleInfo->ModuleSourcePath;
	const FString& NewModulePath = Value->ModuleSourcePath;

	SelectedModuleInfo = Value;

	// Update the class path to be rooted to the new module location
	const FString AbsoluteClassPath = FPaths::ConvertRelativePathToFull(NewClassPath) / ""; // Ensure trailing /
	if(AbsoluteClassPath.StartsWith(OldModulePath))
	{
		NewClassPath = AbsoluteClassPath.Replace(*OldModulePath, *NewModulePath);
	}

	UpdateInputValidity();
}

TSharedRef<SWidget> SNewClassDialog::MakeWidgetForSelectedModuleCombo(TSharedPtr<FModuleContextInfo> Value)
{
	FFormatNamedArguments Args;
	Args.Add(TEXT("ModuleName"), FText::FromString(Value->ModuleName));
	Args.Add(TEXT("ModuleType"), FText::FromString(EHostType::ToString(Value->ModuleType)));
	return SNew(STextBlock)
		.Text(FText::Format(LOCTEXT("ModuleComboEntry", "{ModuleName} ({ModuleType})"), Args));
}

void SNewClassDialog::OnConfirmHotkeyPressed()
{
	// Advance to the name page
	constexpr int32 NamePageIdx = 1;
	if (MainWizard->CanShowPage(NamePageIdx))
	{
		MainWizard->ShowPage(NamePageIdx);
	}
	// Otherwise finish
	else if (CanFinish())
	{
		FinishClicked();
	}
}

FSlateColor SNewClassDialog::GetClassLocationTextColor(GameProjectUtils::EClassLocation InLocation) const
{
	return (ClassLocation == InLocation) ? FSlateColor(FLinearColor(0, 0, 0)) : FSlateColor(FLinearColor(0.72f, 0.72f, 0.72f, 1.f));
}

GameProjectUtils::EClassLocation SNewClassDialog::IsClassLocationActive() const
{
	return ClassLocation;
}

void SNewClassDialog::OnClassLocationChanged(GameProjectUtils::EClassLocation InLocation)
{
	const FString AbsoluteClassPath = FPaths::ConvertRelativePathToFull(NewClassPath) / ""; // Ensure trailing /

	GameProjectUtils::EClassLocation TmpClassLocation = GameProjectUtils::EClassLocation::UserDefined;
	GameProjectUtils::GetClassLocation(AbsoluteClassPath, *SelectedModuleInfo, TmpClassLocation);

	const FString RootPath = SelectedModuleInfo->ModuleSourcePath;
	const FString PublicPath = RootPath / "Public" / "";		// Ensure trailing /
	const FString PrivatePath = RootPath / "Private" / "";		// Ensure trailing /

	// Update the class path to be rooted to the Public or Private folder based on InVisibility
	switch (InLocation)
	{
	case GameProjectUtils::EClassLocation::Public:
		if (AbsoluteClassPath.StartsWith(PrivatePath))
		{
			NewClassPath = AbsoluteClassPath.Replace(*PrivatePath, *PublicPath);
		}
		else if (AbsoluteClassPath.StartsWith(RootPath))
		{
			NewClassPath = AbsoluteClassPath.Replace(*RootPath, *PublicPath);
		}
		else
		{
			NewClassPath = PublicPath;
		}
		break;

	case GameProjectUtils::EClassLocation::Private:
		if (AbsoluteClassPath.StartsWith(PublicPath))
		{
			NewClassPath = AbsoluteClassPath.Replace(*PublicPath, *PrivatePath);
		}
		else if (AbsoluteClassPath.StartsWith(RootPath))
		{
			NewClassPath = AbsoluteClassPath.Replace(*RootPath, *PrivatePath);
		}
		else
		{
			NewClassPath = PrivatePath;
		}
		break;

	default:
		break;
	}

	// Will update ClassVisibility correctly
	UpdateInputValidity();
}

void SNewClassDialog::UpdateInputValidity()
{
	bLastInputValidityCheckSuccessful = true;

	if (ClassDomain == EClassDomain::Blueprint)
	{
		bLastInputValidityCheckSuccessful = GameProjectUtils::IsValidClassNameForCreation(NewClassName, LastInputValidityErrorText);
		ClassLocation = GameProjectUtils::EClassLocation::UserDefined;
		if (bLastInputValidityCheckSuccessful)
		{
			IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(AssetRegistryConstants::ModuleName).Get();
			const FSoftObjectPath ObjectPath(NewClassPath / NewClassName + "." + NewClassName);
			if (AssetRegistry.GetAssetByObjectPath(ObjectPath).IsValid())
			{
				bLastInputValidityCheckSuccessful = false;
				LastInputValidityErrorText = FText::Format(LOCTEXT("AssetAlreadyExists", "An asset called {0} already exists in {1}."), FText::FromString(NewClassName), FText::FromString(NewClassPath));
			}
		}
	}
	else
	{
		// Validate the path first since this has the side effect of updating the UI
		bLastInputValidityCheckSuccessful = GameProjectUtils::CalculateSourcePaths(NewClassPath, *SelectedModuleInfo, CalculatedClassHeaderName, CalculatedClassSourceName, &LastInputValidityErrorText);
		CalculatedClassHeaderName /= ParentClassInfo.GetHeaderFilename(NewClassName);
		CalculatedClassSourceName /= ParentClassInfo.GetSourceFilename(NewClassName);

		// If the source paths check as succeeded, check to see if we're using a Public/Private class
		if(bLastInputValidityCheckSuccessful)
		{
			GameProjectUtils::GetClassLocation(NewClassPath, *SelectedModuleInfo, ClassLocation);

			// We only care about the Public and Private folders
			if(ClassLocation != GameProjectUtils::EClassLocation::Public && ClassLocation != GameProjectUtils::EClassLocation::Private)
			{
				ClassLocation = GameProjectUtils::EClassLocation::UserDefined;
			}
		}
		else
		{
			ClassLocation = GameProjectUtils::EClassLocation::UserDefined;
		}

		// Validate the class name only if the path is valid
		if ( bLastInputValidityCheckSuccessful )
		{
			const TSet<FString>& DisallowedHeaderNames = FSourceCodeNavigation::GetSourceFileDatabase().GetDisallowedHeaderNames();
			bLastInputValidityCheckSuccessful = GameProjectUtils::IsValidClassNameForCreation(NewClassName, *SelectedModuleInfo, DisallowedHeaderNames, LastInputValidityErrorText);
		}

		// Validate that the class is valid for the currently selected module
		// As a project can have multiple modules, this lets us update the class validity as the user changes the target module
		if ( bLastInputValidityCheckSuccessful && ParentClassInfo.BaseClass )
		{
			bLastInputValidityCheckSuccessful = GameProjectUtils::IsValidBaseClassForCreation(ParentClassInfo.BaseClass, *SelectedModuleInfo);
			if ( !bLastInputValidityCheckSuccessful )
			{
				LastInputValidityErrorText = FText::Format(
					LOCTEXT("NewClassError_InvalidBaseClassForModule", "{0} cannot be used as a base class in the {1} module. Please make sure that {0} is API exported."),
					FText::FromString(ParentClassInfo.BaseClass->GetName()),
					FText::FromString(SelectedModuleInfo->ModuleName)
					);
			}
		}
	}

	LastPeriodicValidityCheckTime = FSlateApplication::Get().GetCurrentTime();

	// Since this function was invoked, periodic validity checks should be re-enabled if they were disabled.
	bPreventPeriodicValidityChecksUntilNextChange = false;
}

const FNewClassInfo& SNewClassDialog::GetSelectedParentClassInfo() const
{
	return ParentClassInfo;
}

void SNewClassDialog::SetupParentClassItems(const TArray<FNewClassInfo>& UserSpecifiedFeaturedClasses)
{
	TArray<FNewClassInfo> DefaultFeaturedClasses;
	const TArray<FNewClassInfo>* ArrayToUse = &UserSpecifiedFeaturedClasses;

	// Setup the featured classes list
	if (ArrayToUse->Num() == 0)
	{
		DefaultFeaturedClasses = ClassDomain == EClassDomain::Native ? FFeaturedClasses::AllNativeClasses() : FFeaturedClasses::ActorClasses();
		ArrayToUse = &DefaultFeaturedClasses;
	}

	for (const auto& Featured : *ArrayToUse)
	{
		ParentClassItemsSource.Add( MakeShareable( new FParentClassItem(Featured) ) );
	}
}

void SNewClassDialog::CloseContainingWindow()
{
	TSharedPtr<SWindow> ContainingWindow = FSlateApplication::Get().FindWidgetWindow(AsShared());

	if ( ContainingWindow.IsValid() )
	{
		ContainingWindow->RequestDestroyWindow();
	}
}

#undef LOCTEXT_NAMESPACE
