// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AddToProjectConfig.h"
#include "Containers/Array.h"
#include "Containers/BitArray.h"
#include "Containers/Set.h"
#include "Containers/SparseArray.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "GameProjectUtils.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "Input/Reply.h"
#include "Internationalization/Text.h"
#include "Layout/Visibility.h"
#include "Misc/Optional.h"
#include "Serialization/Archive.h"
#include "Styling/SlateColor.h"
#include "Templates/SharedPointer.h"
#include "Templates/TypeHash.h"
#include "Templates/UnrealTemplate.h"
#include "Types/SlateEnums.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/SListView.h"

class IClassViewerFilter;
class ITableRow;
class SEditableTextBox;
class STableViewBase;
class SWidget;
class SWindow;
class SWizard;
class UClass;
struct FGeometry;
struct FKeyEvent;
struct FModuleContextInfo;
struct FParentClassItem;

enum class EClassDomain : uint8 { Blueprint, Native };

/**
 * A dialog to choose a new class parent and name
 */
class SNewClassDialog : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS( SNewClassDialog )
		: _ClassDomain(EClassDomain::Native)
		, _Class(NULL)
		, _SyncContentBrowserToNewClass(true)
	{}

	/** A reference to the parent window */
	SLATE_ARGUMENT(TSharedPtr<SWindow>, ParentWindow)

	/** The domain of the class we are to create (native or blueprint) */
	SLATE_ARGUMENT(EClassDomain, ClassDomain)

	/** An array of classes to feature on the class picker page */
	SLATE_ARGUMENT(TArray<FNewClassInfo>, FeaturedClasses)

	/** Filter specifying allowable class types, if a parent class is to be chosen by the user */
	SLATE_ARGUMENT(TSharedPtr<IClassViewerFilter>, ClassViewerFilter)

	/** The class we want to build our new class from. If this is not specified then the wizard will display classes to the user. */
	SLATE_ARGUMENT(const UClass*, Class)

	/** The initial path to use as the destination for the new class. If this is not specified, we will work out a suitable default from the available project modules */
	SLATE_ARGUMENT(FString, InitialPath)

	/** The prefix to put on new classes by default, if the user doesn't type in a new name.  Defaults to 'My'. */
	SLATE_ARGUMENT(FString, DefaultClassPrefix)

	/** If non-empty, overrides the default name of the class, when the user doesn't type a new name.  Defaults to empty, which causes the
	    name to be the inherited class name.  Note that DefaultClassPrefix is still prepended to this name, if non-empty. */
	SLATE_ARGUMENT(FString, DefaultClassName)

	/** Event called when code is successfully added to the project */
	SLATE_EVENT( FOnAddedToProject, OnAddedToProject )

	/** Whether we should sync the content browser to focus on the newly created asset */
	SLATE_ARGUMENT(bool, SyncContentBrowserToNewClass)

	SLATE_END_ARGS()

	/** Constructs this widget with InArgs */
	void Construct( const FArguments& InArgs );

	/** Get the widget to focus on activation */
	TSharedPtr<SWidget> GetWidgetToFocusOnActivate();

	virtual void Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime ) override;

	/** Handler for when cancel is clicked */
	void CancelClicked();

	/** Handler for when the Confirm hotkey is pressed */
	void OnConfirmHotkeyPressed();

private:

	/** Creates a row in the parent class list */
	TSharedRef<ITableRow> MakeParentClassListViewWidget(TSharedPtr<FParentClassItem> ParentClassItem, const TSharedRef<STableViewBase>& OwnerTable);

	/** Gets the currently selected parent class name */
	FText GetSelectedParentClassName() const;

	/** Gets the currently selected parent class's filename */
	FText GetSelectedParentClassFilename() const;

	/** Whether the hyper link to go to source should be visible */
	EVisibility GetSourceHyperlinkVisibility() const;

	/** Gets the currently select parent class's doc link */
	FString GetSelectedParentDocLink() const;

	/** Whether the document link anchor should be visible */
	EVisibility GetDocLinkVisibility() const;
	
	/** Handler for when the selected parent class's filename is clicked */
	void OnEditCodeClicked();

	/** Handler for when a parent class item is double clicked */
	void OnParentClassItemDoubleClicked( TSharedPtr<FParentClassItem> TemplateItem );

	/** Handler for when a class is selected in the parent class list */
	void OnClassSelected(TSharedPtr<FParentClassItem> Item, ESelectInfo::Type SelectInfo);

	/** Handler for when a class was picked in the full class tree */
	void OnAdvancedClassSelected(UClass* Class);

	/** Gets the boolean for the full class list */
	bool IsFullClassTreeShown() const;

	/** Sets the boolean for the full class list */
	void OnFullClassTreeChanged(bool bInShowFullClassTree);

	/** Gets the visibility of the basic class list */
	EVisibility GetBasicParentClassVisibility() const;

	/** Gets the visibility of the full class list */
	EVisibility GetAdvancedParentClassVisibility() const;

	/** Gets the visibility of the name error label */
	EVisibility GetNameErrorLabelVisibility() const;

	/** Gets the text to display in the name error label */
	FText GetNameErrorLabelText() const;

	/** Gets the visibility of the global error label */
	EVisibility GetGlobalErrorLabelVisibility() const;

	/** Gets the text to display in the global error label */
	FText GetGlobalErrorLabelText() const;

	/** Handler for when the user enters the "name class" page */
	void OnNamePageEntered();

	/** Returns the title text for the "name class" page */
	FText GetNameClassTitle() const;

	/** Returns the text in the class name edit box */
	FText OnGetClassNameText() const;

	/** Handler for when the text in the class name edit box has changed */
	void OnClassNameTextChanged(const FText& NewText);

	/** Handler for when the text in the class name edit box is committed */
	void OnClassNameTextCommitted(const FText& NewText, ETextCommit::Type CommitType);

	/** Returns the text in the class path edit box */
	FText OnGetClassPathText() const;

	/** Handler for when the text in the class path edit box has changed */
	void OnClassPathTextChanged(const FText& NewText);

	/** Called when the user chooses a path for a blueprint */
	void OnBlueprintPathSelected(const FString& NewPath);

	/** Returns the text for the calculated header file name */
	FText OnGetClassHeaderFileText() const;

	/** Returns the text for the calculated source file name */
	FText OnGetClassSourceFileText() const;

	/** Returns true if Finish is allowed */
	bool CanFinish() const;

	/** Handler for when finish is clicked */
	void FinishClicked();

	/** Handler for when the "Choose Folder" button is clicked */
	FReply HandleChooseFolderButtonClicked( );

	/** Get the combo box text for the currently selected module */
	FText GetSelectedModuleComboText() const;

	/** Called when the currently selected module is changed */
	void SelectedModuleComboBoxSelectionChanged(TSharedPtr<FModuleContextInfo> Value, ESelectInfo::Type SelectInfo);

	/** Create the widget to use as the combo box entry for the given module info */
	TSharedRef<SWidget> MakeWidgetForSelectedModuleCombo(TSharedPtr<FModuleContextInfo> Value);

private:

	/** Get the text color to use for the given class location checkbox */
	FSlateColor GetClassLocationTextColor(GameProjectUtils::EClassLocation InLocation) const;

	/** Checks to see if the given class location is active based on the current value of NewClassPath */
	GameProjectUtils::EClassLocation IsClassLocationActive() const;

	/** Update the value of NewClassPath so that it uses the given class location */
	void OnClassLocationChanged(GameProjectUtils::EClassLocation InLocation);

	/** Checks the current class name/path for validity and updates cached values accordingly */
	void UpdateInputValidity();

	/** Gets the currently selected parent class */
	const FNewClassInfo& GetSelectedParentClassInfo() const;

	/** Adds parent classes to the ParentClassListView source */
	void SetupParentClassItems(const TArray<FNewClassInfo>& UserSpecifiedFeaturedClasses);

	/** Closes the window that contains this widget */
	void CloseContainingWindow();

private:

	/** The wizard widget */
	TSharedPtr<SWizard> MainWizard;
	
	/** ParentClass items */
	TSharedPtr< SListView< TSharedPtr<FParentClassItem> > > ParentClassListView;
	TArray< TSharedPtr<FParentClassItem> > ParentClassItemsSource;

	/** A pointer to a class viewer **/
	TSharedPtr<class SClassViewer> ClassViewer;

	/** The prefix to put on new classes by default, if the user doesn't type in a new name.  Defaults to 'My'. */
	FString DefaultClassPrefix;

	/** If non-empty, overrides the default name of the class, when the user doesn't type a new name.  Defaults to empty, which causes the
	    name to be the inherited class name.  Note that DefaultClassPrefix is still prepended to this name, if non-empty. */
	FString DefaultClassName;

	/** The editable text box to enter the current name */
	TSharedPtr<SEditableTextBox> ClassNameEditBox;

	/** The available modules combo box */
	TSharedPtr<SComboBox<TSharedPtr<FModuleContextInfo>>> AvailableModulesCombo;

	/** The last selected module name. Meant to keep the same module selected after first selection */
	static FString LastSelectedModuleName;

	/** The name of the class being created */
	FString NewClassName;

	/** The path to place the files for the class being generated */
	FString NewClassPath;

	/** The calculated name of the generated header file for this class */
	FString CalculatedClassHeaderName;

	/** The calculated name of the generated source file for this class */
	FString CalculatedClassSourceName;

	/** The name of the last class that was auto-generated by this wizard */
	FString LastAutoGeneratedClassName;

	/** The selected parent class */
	FNewClassInfo ParentClassInfo;

	/** If true, the full class tree will be shown in the parent class selection */
	bool bShowFullClassTree;

	/** The last time that the class name/path was checked for validity. This is used to throttle I/O requests to a reasonable frequency */
	double LastPeriodicValidityCheckTime;

	/** The frequency in seconds for validity checks while the dialog is idle. Changes to the name/path immediately update the validity. */
	double PeriodicValidityCheckFrequency;

	/** Periodic checks for validity will not occur while this flag is true. Used to prevent a frame of "this project already exists" while exiting after a successful creation. */
	bool bPreventPeriodicValidityChecksUntilNextChange;

	/** The error text from the last validity check */
	FText LastInputValidityErrorText;

	/** True if the last validity check returned that the class name/path is valid for creation */
	bool bLastInputValidityCheckSuccessful;

	/** Whether the class should be created as a Public or Private class */
	GameProjectUtils::EClassLocation ClassLocation;

	/** The domain of the new class we are creating (native or blueprint) */
	EClassDomain ClassDomain;

	/** Information about the currently available modules for this project */
	TArray<TSharedPtr<FModuleContextInfo>> AvailableModules;

	/** Information about the currently selected module; used for class validation */
	TSharedPtr<FModuleContextInfo> SelectedModuleInfo;

	/** Event called when code is succesfully added to the project */
	FOnAddedToProject OnAddedToProject;

	/** Flag set when PathPicker selects a path*/
	bool IsBlueprintPathSelected = false;

	/** Whether we should sync the content browser to navigate to the newly created asset */
	bool bSyncContentBrowserToNewClass = true;
};
