// Copyright Epic Games, Inc. All Rights Reserved.


#include "SProjectBrowser.h"

#include "AnalyticsEventAttribute.h"
#include "Brushes/SlateDynamicImageBrush.h"
#include "Containers/EnumAsByte.h"
#include "Containers/Map.h"
#include "CoreGlobals.h"
#include "DesktopPlatformModule.h"
#include "EditorDirectories.h"
#include "EngineAnalytics.h"
#include "Fonts/SlateFontInfo.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/Commands/UICommandInfo.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/Views/ITypedTableView.h"
#include "GameProjectUtils.h"
#include "GenericPlatform/GenericWindow.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformMisc.h"
#include "HAL/PlatformProcess.h"
#include "IAnalyticsProviderET.h"
#include "IDesktopPlatform.h"
#include "ILauncherPlatform.h"
#include "Input/Events.h"
#include "InputCoreTypes.h"
#include "Interfaces/IMainFrameModule.h"
#include "Interfaces/IProjectManager.h"
#include "Internationalization/BreakIterator.h"
#include "Internationalization/Internationalization.h"
#include "LauncherPlatformModule.h"
#include "Layout/Children.h"
#include "Layout/Margin.h"
#include "Logging/LogCategory.h"
#include "Logging/LogMacros.h"
#include "Math/Vector2D.h"
#include "Math/Vector4.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "Misc/DataDrivenPlatformInfoRegistry.h"
#include "Misc/DateTime.h"
#include "Misc/EngineVersion.h"
#include "Misc/EngineVersionBase.h"
#include "Misc/FeedbackContext.h"
#include "Misc/MessageDialog.h"
#include "Misc/OutputDeviceRedirector.h"
#include "Misc/Paths.h"
#include "Misc/StringOutputDevice.h"
#include "Misc/UProjectInfo.h"
#include "Modules/ModuleManager.h"
#include "PlatformInfo.h"
#include "ProjectDescriptor.h"
#include "SSimpleButton.h"
#include "SVerbChoiceDialog.h"
#include "Settings/EditorSettings.h"
#include "SlotBase.h"
#include "SourceCodeNavigation.h"
#include "Styling/AppStyle.h"
#include "Styling/ISlateStyle.h"
#include "Styling/SlateBrush.h"
#include "Styling/SlateColor.h"
#include "Styling/SlateTypes.h"
#include "Styling/StyleColors.h"
#include "Styling/StyleDefaults.h"
#include "Textures/SlateIcon.h"
#include "Trace/Detail/Channel.h"
#include "Types/SlateStructs.h"
#include "UObject/Class.h"
#include "UObject/NameTypes.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UnrealNames.h"
#include "UObject/UnrealType.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SOverlay.h"
#include "Widgets/SToolTip.h"
#include "Widgets/SWindow.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STileView.h"

class SWidget;
struct FGeometry;

#define LOCTEXT_NAMESPACE "ProjectBrowser"

DEFINE_LOG_CATEGORY_STATIC(LogProjectBrowser, Log, All);

namespace ProjectBrowserDefs
{
	constexpr float ProjectTileHeight = 153;
	constexpr float ProjectTileWidth = 102;
	constexpr float ThumbnailSize = 64.0f, ThumbnailPadding = 5.f;
}


/**
 * Structure for project items.
 */
struct FProjectItem
{
	FText Name;
	FText Description;
	FText Category;
	FString EngineIdentifier;
	FEngineVersion ProjectVersion;
	FString ProjectFile;
	TArray<FName> TargetPlatforms;
	TSharedPtr<FSlateBrush> ProjectThumbnail;
	FDateTime LastAccessTime;
	bool bUpToDate;
	bool bSupportsAllPlatforms;

	FProjectItem(const FText& InName, const FText& InDescription, const FString& InEngineIdentifier, bool InUpToDate, const TSharedPtr<FSlateBrush>& InProjectThumbnail, const FString& InProjectFile, TArray<FName> InTargetPlatforms, bool InSupportsAllPlatforms)
		: Name(InName)
		, Description(InDescription)
		, Category()
		, EngineIdentifier(InEngineIdentifier)
		, ProjectFile(InProjectFile)
		, TargetPlatforms(InTargetPlatforms)
		, ProjectThumbnail(InProjectThumbnail)
		, bUpToDate(InUpToDate)
		, bSupportsAllPlatforms(InSupportsAllPlatforms)
	{
		if(bUpToDate)
		{
			ProjectVersion = FEngineVersion::Current();
		}
		else if (FDesktopPlatformModule::Get()->IsStockEngineRelease(EngineIdentifier))
		{
			FDesktopPlatformModule::Get()->TryParseStockEngineVersion(EngineIdentifier, ProjectVersion);
		}


		if (ProjectVersion.IsEmpty())
		{
			FString RootDir;
			FDesktopPlatformModule::Get()->GetEngineRootDirFromIdentifier(EngineIdentifier, RootDir);
			FDesktopPlatformModule::Get()->TryGetEngineVersion(RootDir, ProjectVersion);
		}
	}

	/** Check if this project is up to date */
	bool IsUpToDate() const
	{
		return bUpToDate;
	}

	/** Gets the engine label for this project */
	FText GetEngineLabel() const
	{
		if(bUpToDate)
		{
			return LOCTEXT("CurrentEngineVersion", "Current");
		}
		else if(FDesktopPlatformModule::Get()->IsStockEngineRelease(EngineIdentifier))
		{
			return FText::FromString(EngineIdentifier);
		}
		else if(!ProjectVersion.IsEmpty())
		{
			return FText::FromString(ProjectVersion.ToString(EVersionComponent::Patch));
		}
		else
		{
			return FText::FromString(TEXT("?"));
		}
	}

	bool CompareEngineVersion(const FProjectItem& Other) const
	{
		EVersionComponent Component;
		return FEngineVersion::GetNewest(ProjectVersion, Other.ProjectVersion, &Component) == EVersionComparison::First ? true : false;
	}
};


class SProjectTile : public STableRow<TSharedPtr<FProjectItem>>
{
	SLATE_BEGIN_ARGS(SProjectTile)
	{}
	SLATE_END_ARGS()

public:
	void Construct(const FArguments& InArgs, TSharedPtr<FProjectItem> ProjectItem, const TSharedRef<STableViewBase>& OwnerTable)
	{
		STableRow::Construct(
			STableRow::FArguments()
			.Style(FAppStyle::Get(), "ProjectBrowser.TableRow")
			.Padding(2.0f)
			.Content()
		[
			SNew(SBorder)
			.Padding(FMargin(0.0f, 0.0f, 5.0f, 5.0f))
			.BorderImage(FAppStyle::Get().GetBrush("ProjectBrowser.ProjectTile.DropShadow"))
			[
				SNew(SOverlay)
				+ SOverlay::Slot()
				[
					SNew(SVerticalBox)
					// Thumbnail
					+ SVerticalBox::Slot()
					.AutoHeight()
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					[
						SNew(SBox)
						.WidthOverride(102.0f)
						.HeightOverride(102.0f)
						[
							SNew(SBorder)
							.BorderImage(FAppStyle::Get().GetBrush("ProjectBrowser.ProjectTile.ThumbnailAreaBackground"))
							.HAlign(HAlign_Fill)
							.VAlign(VAlign_Fill)
							.Padding(FMargin(0))
							[
								SNew(SImage)
								.Image(ProjectItem->ProjectThumbnail ? ProjectItem->ProjectThumbnail.Get() : FAppStyle::Get().GetBrush("UnrealDefaultThumbnailRounded"))
								.ColorAndOpacity(FAppStyle::Get().GetSlateColor("Colors.Foreground"))
							]
						]
					]
					// Name
					+ SVerticalBox::Slot()
					[
						SNew(SBorder)
						.Padding(FMargin(ProjectBrowserDefs::ThumbnailPadding, 0))
						.Padding(FMargin(3.0f, 3.0f))
						.BorderImage_Lambda
						(
							[this]()
							{
								const bool bIsSelected = IsSelected();
								const bool bIsRowHovered = IsHovered();

								if (bIsSelected && bIsRowHovered)
								{
									static const FName SelectedHover("ProjectBrowser.ProjectTile.NameAreaSelectedHoverBackground");
									return FAppStyle::Get().GetBrush(SelectedHover);
								}
								else if (bIsSelected)
								{
									static const FName Selected("ProjectBrowser.ProjectTile.NameAreaSelectedBackground");
									return FAppStyle::Get().GetBrush(Selected);
								}
								else if (bIsRowHovered)
								{
									static const FName Hovered("ProjectBrowser.ProjectTile.NameAreaHoverBackground");
									return FAppStyle::Get().GetBrush(Hovered);
								}

								return FAppStyle::Get().GetBrush("ProjectBrowser.ProjectTile.NameAreaBackground");
							}
						)
						[
							SNew(SVerticalBox)
							+SVerticalBox::Slot()
							[
								SNew(STextBlock)
								.Font(FAppStyle::Get().GetFontStyle("ProjectBrowser.ProjectTile.Font"))
								.WrapTextAt(ProjectBrowserDefs::ProjectTileWidth - 4.0f)
								.LineBreakPolicy(FBreakIterator::CreateCamelCaseBreakIterator())
								.Text(ProjectItem->Name)
								.ColorAndOpacity_Lambda
								(
									[this]()
									{
										const bool bIsSelected = IsSelected();
										const bool bIsRowHovered = IsHovered();

										if (bIsSelected || bIsRowHovered)
										{
											return FStyleColors::White;
										}

										return FSlateColor::UseForeground();
									}
								)
							]
							+SVerticalBox::Slot()
							.AutoHeight()
							.Padding(0.0f, 4.0f, 0.0f, 0.0f)
							.VAlign(VAlign_Bottom)
							[
								SNew(STextBlock)
								.Text(ProjectItem->GetEngineLabel())
								.Font(FAppStyle::Get().GetFontStyle("ProjectBrowser.ProjectTile.Font"))
								.ColorAndOpacity_Lambda
								(
									[this]()
									{
										const bool bIsSelected = IsSelected();
										const bool bIsRowHovered = IsHovered();

										if (bIsSelected || bIsRowHovered)
										{
											return FStyleColors::White;
										}

										return FSlateColor::UseSubduedForeground();
									}
								)
							]
						]
					]
				]
				+ SOverlay::Slot()
				[
					SNew(SImage)
					.Visibility(EVisibility::HitTestInvisible)
					.Image_Lambda
					(
						[this]()
						{
							const bool bIsSelected = IsSelected();
							const bool bIsRowHovered = IsHovered();

							if (bIsSelected && bIsRowHovered)
							{
								static const FName SelectedHover("ProjectBrowser.ProjectTile.SelectedHoverBorder");
								return FAppStyle::Get().GetBrush(SelectedHover);
							}
							else if (bIsSelected)
							{
								static const FName Selected("ProjectBrowser.ProjectTile.SelectedBorder");
								return FAppStyle::Get().GetBrush(Selected);
							}
							else if (bIsRowHovered)
							{
								static const FName Hovered("ProjectBrowser.ProjectTile.HoverBorder");
								return FAppStyle::Get().GetBrush(Hovered);
							}

							return FStyleDefaults::GetNoBrush();
						}
					)
				]
			]
		],
		OwnerTable);
	}
};

void ProjectItemToString(const TSharedPtr<FProjectItem> InItem, TArray<FString>& OutFilterStrings)
{
	OutFilterStrings.Add(InItem->Name.ToString());
}


/* SCompoundWidget interface
 *****************************************************************************/

SProjectBrowser::SProjectBrowser()
	: ProjectItemFilter( ProjectItemTextFilter::FItemToStringArray::CreateStatic( ProjectItemToString ))
	, bPreventSelectionChangeEvent(false)
{

}

void SProjectBrowser::Construct( const FArguments& InArgs )
{
	TRACE_CPUPROFILER_EVENT_SCOPE(SProjectBrowser::Construct);

	// Prepare the projects box
	ProjectsBox = SNew(SVerticalBox);

	// If embedded the HeaderBar and AllowScrolling should not be allowed
	bIsEmbedded = InArgs._IsEmbedded;
	const bool ShowHeaderBar = InArgs._ShowHeaderBar && !bIsEmbedded;
	const bool AllowScrolling = InArgs._AllowScrolling && !bIsEmbedded;

	SAssignNew(ProjectTileView, STileView<TSharedPtr<FProjectItem>>)
		.ListItemsSource(&FilteredProjectItemsSource)
		.SelectionMode(ESelectionMode::Single)
		.ClearSelectionOnClick(false)
		.ItemAlignment(EListItemAlignment::LeftAligned)
		.OnGenerateTile(this, &SProjectBrowser::MakeProjectViewWidget)
		.OnContextMenuOpening(this, &SProjectBrowser::OnGetContextMenuContent)
		.OnMouseButtonDoubleClick(this, &SProjectBrowser::HandleProjectItemDoubleClick)
		.OnSelectionChanged(this, &SProjectBrowser::HandleProjectViewSelectionChanged)
		.AllowOverscroll(AllowScrolling ? EAllowOverscroll::Yes : EAllowOverscroll::No)
		.ConsumeMouseWheel(AllowScrolling ? EConsumeMouseWheel::Always : EConsumeMouseWheel::Never)
		.ScrollbarVisibility(AllowScrolling ? EVisibility::Visible : EVisibility::Collapsed)
		.WheelScrollMultiplier(AllowScrolling ? GetGlobalScrollAmount() : 0.f)
		.ItemHeight(ProjectBrowserDefs::ProjectTileHeight+9)
		.ItemWidth(ProjectBrowserDefs::ProjectTileWidth+9);

	// Find all projects
	FindProjects();

	ProjectsBox->AddSlot()
	.HAlign(HAlign_Center)
	.Padding(FMargin(0.f, 25.f))
	[
		SNew(STextBlock)
		.Visibility(this, &SProjectBrowser::GetNoProjectsErrorVisibility)
		.Text(LOCTEXT("NoProjects", "You don't have any projects yet :("))
	];

	ProjectsBox->AddSlot()
	.HAlign(HAlign_Center)
	.Padding(FMargin(0.f, 25.f))
	[
		SNew(STextBlock)
		.Visibility(this, &SProjectBrowser::GetNoProjectsAfterFilterErrorVisibility)
		.Text(LOCTEXT("NoProjectsAfterFilter", "There are no projects that match the specified filter"))
	];

	ProjectsBox->AddSlot()
	[
		ProjectTileView.ToSharedRef()
	];

	FMenuBuilder SortOptionsBuilder(true, nullptr);

	auto MakeSortMenuEntry = [this, &SortOptionsBuilder](EProjectSortOption Option, FText Label, FText Tooltip)
	{
		FUIAction Action;
	
		Action.ExecuteAction = FExecuteAction::CreateSP(this, &SProjectBrowser::SortProjectTiles, Option);
		Action.GetActionCheckState = FGetActionCheckState::CreateSP(this, &SProjectBrowser::GetSortOptionCheckState, Option);
		SortOptionsBuilder.AddMenuEntry(Label, Tooltip, FSlateIcon(), Action, NAME_None, EUserInterfaceActionType::RadioButton);
		
	};

	MakeSortMenuEntry(EProjectSortOption::LastAccessTime, LOCTEXT("ProjectSortMode_LastAcessTime", "Most Recently Used"), FText::GetEmpty());
	MakeSortMenuEntry(EProjectSortOption::Alphabetical, LOCTEXT("ProjectSortMode_Alphabetical", "Alphabetical"), FText::GetEmpty());
	MakeSortMenuEntry(EProjectSortOption::Version, LOCTEXT("ProjectSortMode_Version", "Version"), FText::GetEmpty());


	ChildSlot
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			.Visibility(ShowHeaderBar ? EVisibility::Visible : EVisibility::Collapsed)

			+ SHorizontalBox::Slot()
			.Padding(FMargin(0, 0, 5.f, 5))
			.VAlign(VAlign_Center)
			[
				SAssignNew(SearchBoxPtr, SSearchBox)
				.HintText(LOCTEXT("FilterHint", "Filter Projects..."))
				.OnTextChanged(this, &SProjectBrowser::OnFilterTextChanged)
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(FMargin(0, 0, 5.f, 5))
			[
				SNew(SComboButton)
				.ToolTipText(LOCTEXT("SortProjectList", "Sort the project list"))
				.ComboButtonStyle(FAppStyle::Get(), "SimpleComboButtonWithIcon")
				.MenuContent()
				[
					SortOptionsBuilder.MakeWidget()
				]
				.ButtonContent()
				[
					SNew(SImage)
					.Image(FAppStyle::GetBrush("Icons.SortDown"))
					.ColorAndOpacity(FSlateColor::UseForeground())
				]

			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(FMargin(0, 0, 5.f, 5))
			[
				SNew(SSimpleButton)
				.OnClicked(this, &SProjectBrowser::FindProjects)
				.Icon(FAppStyle::GetBrush("Icons.Refresh"))
				.ToolTipText(LOCTEXT("RefreshProjectList", "Refresh the project list"))
			]
		]
		+ SVerticalBox::Slot()
		.Padding(FMargin(0, 5.f))
		[
			ProjectsBox.ToSharedRef()
		]
	];

	ProjectSelectionChangedDelegate = InArgs._OnSelectionChanged;
}

bool SProjectBrowser::HasProjects() const
{
	return ProjectItemsSource.Num() > 0;
}

FString SProjectBrowser::GetSelectedProjectFile() const
{	
	TSharedPtr<FProjectItem> SelectedItem = GetSelectedProjectItem();
	if (SelectedItem.IsValid())
	{
		return SelectedItem->ProjectFile;
	}

	return FString();
}

/* SProjectBrowser implementation
 *****************************************************************************/

TSharedRef<ITableRow> SProjectBrowser::MakeProjectViewWidget(TSharedPtr<FProjectItem> ProjectItem, const TSharedRef<STableViewBase>& OwnerTable)
{
	return SNew(SProjectTile, ProjectItem, OwnerTable)
			.ToolTip(MakeProjectToolTip(ProjectItem));
}


TSharedRef<SToolTip> SProjectBrowser::MakeProjectToolTip( TSharedPtr<FProjectItem> ProjectItem ) const
{
	// Create a box to hold every line of info in the body of the tooltip
	TSharedRef<SVerticalBox> InfoBox = SNew(SVerticalBox);

	if(!ProjectItem->Description.IsEmpty())
	{
		AddToToolTipInfoBox( InfoBox, LOCTEXT("ProjectTileTooltipDescription", "Description"), ProjectItem->Description );
	}

	{
		const FString ProjectPath = FPaths::GetPath(ProjectItem->ProjectFile);
		AddToToolTipInfoBox( InfoBox, LOCTEXT("ProjectTileTooltipPath", "Path"), FText::FromString(ProjectPath) );
	}

	{
		FText Description;
		if(FDesktopPlatformModule::Get()->IsStockEngineRelease(ProjectItem->EngineIdentifier))
		{
			Description = FText::FromString(ProjectItem->EngineIdentifier);
		}
		else
		{
			FString RootDir;
			if(FDesktopPlatformModule::Get()->GetEngineRootDirFromIdentifier(ProjectItem->EngineIdentifier, RootDir))
			{
				FString PlatformRootDir = RootDir;
				FPaths::MakePlatformFilename(PlatformRootDir);
				Description = FText::FromString(PlatformRootDir);
			}
			else
			{
				Description = LOCTEXT("UnknownEngineVersion", "Unknown engine version");
			}
		}
		AddToToolTipInfoBox(InfoBox, LOCTEXT("EngineVersion", "Engine"), Description);
	}

	// Create the target platform icons
	TSharedRef<SHorizontalBox> TargetPlatformIconsBox = SNew(SHorizontalBox);
	for(const FName& PlatformName : ProjectItem->TargetPlatforms)
	{
		const PlatformInfo::FTargetPlatformInfo* const PlatformInfo = PlatformInfo::FindPlatformInfo(PlatformName);
		check(PlatformInfo);

		TargetPlatformIconsBox->AddSlot()
		.AutoWidth()
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		.Padding(FMargin(0.0f, 0.0f, 1.0f, 0.0f))
		[
			SNew(SBox)
			.WidthOverride(20.0f)
			.HeightOverride(20.0f)
			[
				SNew(SImage)
				.Image(FAppStyle::GetBrush(PlatformInfo->GetIconStyleName(EPlatformIconSize::Normal)))
			]
		];
	}

	TSharedRef<SToolTip> Tooltip = SNew(SToolTip)
	.TextMargin(1.0f)
	.BorderImage( FAppStyle::GetBrush("ProjectBrowser.TileViewTooltip.ToolTipBorder") )
	[
		SNew(SBorder)
		.Padding(6.0f)
		.BorderImage( FAppStyle::GetBrush("ProjectBrowser.TileViewTooltip.NonContentBorder") )
		[
			SNew(SVerticalBox)

			+SVerticalBox::Slot()
			.AutoHeight()
			.Padding(0.0f, 0.0f, 0.0f, 4.0f)
			[
				SNew(SBorder)
				.Padding(6.0f)
				.BorderImage( FAppStyle::GetBrush("ProjectBrowser.TileViewTooltip.ContentBorder") )
				[
					SNew(SVerticalBox)

					+SVerticalBox::Slot()
					.AutoHeight()
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
						.Text( ProjectItem->Name )
						.Font( FAppStyle::GetFontStyle("ProjectBrowser.TileViewTooltip.NameFont") )
					]

					+SVerticalBox::Slot()
					.AutoHeight()
					.VAlign(VAlign_Center)
					.Padding(0.0f, 2.0f, 0.0f, 0.0f)
					[
						TargetPlatformIconsBox
					]
				]
			]

			+SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SBorder)
				.Padding(6.0f)
				.BorderImage( FAppStyle::GetBrush("ProjectBrowser.TileViewTooltip.ContentBorder") )
				[
					InfoBox
				]
			]
		]
	];

	return Tooltip;
}


void SProjectBrowser::AddToToolTipInfoBox(const TSharedRef<SVerticalBox>& InfoBox, const FText& Key, const FText& Value) const
{
	InfoBox->AddSlot()
	.AutoHeight()
	.Padding(0.0f, 1.0f)
	[
		SNew(SHorizontalBox)

		+SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(0.0f, 0.0f, 4.0f, 0.0f)
		[
			SNew(STextBlock) .Text( FText::Format(LOCTEXT("ProjectBrowserTooltipFormat", "{0}:"), Key ) )
			.ColorAndOpacity(FSlateColor::UseSubduedForeground())
		]

		+SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew(STextBlock) .Text( Value )
			.ColorAndOpacity(FSlateColor::UseForeground())
		]
	];
}


TSharedPtr<SWidget> SProjectBrowser::OnGetContextMenuContent() const
{
	FMenuBuilder MenuBuilder(/*bInShouldCloseWindowAfterMenuSelection=*/true, nullptr);

	TSharedPtr<FProjectItem> SelectedProjectItem = GetSelectedProjectItem();
	const FText ProjectContextActionsText = (SelectedProjectItem.IsValid()) ? SelectedProjectItem->Name : LOCTEXT("ProjectActionsMenuHeading", "Project Actions");
	MenuBuilder.BeginSection("ProjectContextActions", ProjectContextActionsText);

	FFormatNamedArguments Args;
	Args.Add(TEXT("FileManagerName"), FPlatformMisc::GetFileManagerName());
	const FText ExploreToText = FText::Format(NSLOCTEXT("GenericPlatform", "ShowInFileManager", "Show in {FileManagerName}"), Args);

	MenuBuilder.AddMenuEntry(
		ExploreToText,
		LOCTEXT("FindInExplorerTooltip", "Finds this project on disk"),
		FSlateIcon(),
		FUIAction(
		FExecuteAction::CreateSP( this, &SProjectBrowser::ExecuteFindInExplorer ),
		FCanExecuteAction::CreateSP( this, &SProjectBrowser::CanExecuteFindInExplorer )
		)
		);

	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}


void SProjectBrowser::ExecuteFindInExplorer() const
{
	TSharedPtr<FProjectItem> SelectedProjectItem = GetSelectedProjectItem();
	check(SelectedProjectItem.IsValid());
	FPlatformProcess::ExploreFolder(*SelectedProjectItem->ProjectFile);
}


bool SProjectBrowser::CanExecuteFindInExplorer() const
{
	TSharedPtr<FProjectItem> SelectedProjectItem = GetSelectedProjectItem();
	return SelectedProjectItem.IsValid();
}

TSharedPtr<FProjectItem> SProjectBrowser::GetSelectedProjectItem() const
{
	TArray< TSharedPtr<FProjectItem> > SelectedItems = ProjectTileView->GetSelectedItems();
	if (SelectedItems.Num() > 0)
	{
		return SelectedItems[0];	
	}
	
	return nullptr;
}


FText SProjectBrowser::GetSelectedProjectName() const
{
	TSharedPtr<FProjectItem> SelectedItem = GetSelectedProjectItem();
	if ( SelectedItem.IsValid() )
	{
		return SelectedItem->Name;
	}

	return FText::GetEmpty();
}

static TSharedPtr<FSlateDynamicImageBrush> GetThumbnailForProject(const FString& ProjectFilename)
{
	TSharedPtr<FSlateDynamicImageBrush> DynamicBrush;
	const FString ThumbnailPNGFile = FPaths::GetBaseFilename(ProjectFilename, false) + TEXT(".png");
	const FString AutoScreenShotPNGFile = FPaths::Combine(*FPaths::GetPath(ProjectFilename), TEXT("Saved"), TEXT("AutoScreenshot.png"));
	FString PNGFileToUse;
	if (FPaths::FileExists(ThumbnailPNGFile))
	{
		PNGFileToUse = ThumbnailPNGFile;
	}
	else if (FPaths::FileExists(AutoScreenShotPNGFile))
	{
		PNGFileToUse = AutoScreenShotPNGFile;
	}

	if (!PNGFileToUse.IsEmpty())
	{
		const FName BrushName = FName(*PNGFileToUse);
		DynamicBrush = MakeShared<FSlateDynamicImageBrush>(BrushName, FVector2D(128, 128));
		DynamicBrush->OutlineSettings.CornerRadii = FVector4(4, 4, 0, 0);
		DynamicBrush->OutlineSettings.RoundingType = ESlateBrushRoundingType::FixedRadius;
		DynamicBrush->DrawAs = ESlateBrushDrawType::RoundedBox;

	}

	return DynamicBrush;
}

static TSharedPtr<FProjectItem> CreateProjectItem(const FString& ProjectFilename)
{
	if (FPaths::FileExists(ProjectFilename))
	{
		FProjectStatus ProjectStatus;
		if (IProjectManager::Get().QueryStatusForProject(ProjectFilename, ProjectStatus))
		{
			// @todo localized project name
			const FText ProjectName = FText::FromString(ProjectStatus.Name);
			const FText ProjectDescription = FText::FromString(ProjectStatus.Description);

			TSharedPtr<FSlateDynamicImageBrush> DynamicBrush = GetThumbnailForProject(ProjectFilename);

			const FString EngineIdentifier = FDesktopPlatformModule::Get()->GetCurrentEngineIdentifier();

			FString ProjectEngineIdentifier;
			bool bIsUpToDate = FDesktopPlatformModule::Get()->GetEngineIdentifierForProject(ProjectFilename, ProjectEngineIdentifier) 
				&& ProjectEngineIdentifier == EngineIdentifier;

			// Work out which platforms this project is targeting
			TArray<FName> TargetPlatforms;
			for (const PlatformInfo::FTargetPlatformInfo* PlatformInfo : PlatformInfo::GetVanillaPlatformInfoArray())
			{
				if (ProjectStatus.IsTargetPlatformSupported(PlatformInfo->Name))
				{
					TargetPlatforms.Add(PlatformInfo->Name);
				}
			}
			TargetPlatforms.Sort(FNameLexicalLess());

			TSharedPtr<FProjectItem> ProjectItem = MakeShareable(new FProjectItem(FText::FromString(ProjectStatus.Name),
				FText::FromString(ProjectStatus.Description),
				ProjectEngineIdentifier, bIsUpToDate, DynamicBrush, ProjectFilename, 
				TargetPlatforms, ProjectStatus.SupportsAllPlatforms()));

			const FText SamplesCategoryName = LOCTEXT("SamplesCategoryName", "Samples");
			if (ProjectStatus.bSignedSampleProject)
			{
				// Signed samples can't override their category name
				ProjectItem->Category = SamplesCategoryName;
			}

			ProjectItem->LastAccessTime = IFileManager::Get().GetAccessTimeStamp(*ProjectFilename);

			return ProjectItem;
		}
	}

	return nullptr;
}

FReply SProjectBrowser::FindProjects()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(SProjectBrowser::FindProjects);

	const FString LastSelectedProjectFile = CurrentSelectedProjectPath;

	ProjectItemsSource.Empty();
	FilteredProjectItemsSource.Empty();
	ProjectTileView->RequestListRefresh();

	TArray<FRecentProjectFile> RecentProjects = GetDefault<UEditorSettings>()->RecentlyOpenedProjectFiles;

	TSet<FString> AllFoundProjectFiles;

	// Find all the engine installations
	TMap<FString, FString> EngineInstallations;
	FDesktopPlatformModule::Get()->EnumerateEngineInstallations(EngineInstallations);

	UE_LOGF(LogProjectBrowser, Log, "Looking for projects...");

	// Add projects from every branch that we know about
	for (TMap<FString, FString>::TConstIterator Iter(EngineInstallations); Iter; ++Iter)
	{
		TArray<FString> ProjectFiles;

		UE_LOGF(LogProjectBrowser, Log, "Found Engine Installation \"%ls\"(%ls)", *Iter.Key(), *Iter.Value());

		if (FDesktopPlatformModule::Get()->EnumerateProjectsKnownByEngine(Iter.Key(), false, ProjectFiles))
		{
			AllFoundProjectFiles.Append(MoveTemp(ProjectFiles));
		}
	}

	// Add all the samples from the launcher
	TArray<FString> LauncherSampleProjects;
	FDesktopPlatformModule::Get()->EnumerateLauncherSampleProjects(LauncherSampleProjects);

	UE_LOGF(LogProjectBrowser, Log, "Enumerating Launcher Sample Projects...");

	for (const FString& Str : LauncherSampleProjects)
	{
		UE_LOGF(LogProjectBrowser, Log, "Found Sample Project:\"%ls\"", *Str);
	}

	AllFoundProjectFiles.Append(MoveTemp(LauncherSampleProjects));

	// Add all the native project files we can find
	FUProjectDictionary& DefaultProjectDictionary = FUProjectDictionary::GetDefault();
	DefaultProjectDictionary.Refresh();
	const TArray<FString>& NativeProjectFiles = DefaultProjectDictionary.GetProjectPaths();
	for(const FString& ProjectFile : NativeProjectFiles)
	{
		if (!ProjectFile.Contains(TEXT("/Templates/")) && !ProjectFile.Contains(TEXT("/Programs/")))
		{
			AllFoundProjectFiles.Add(ProjectFile);
		}
	}

	TSharedPtr<FProjectItem> NewProjectToSelect;

	// Normalize all the filenames and make sure there are no duplicates
	for (FString& ProjectFile : AllFoundProjectFiles)
	{
		FString ProjectFilename = IFileManager::Get().ConvertToAbsolutePathForExternalAppForRead(*ProjectFile);
		TSharedPtr<FProjectItem> NewProjectItem = CreateProjectItem(ProjectFilename);
		if (NewProjectItem)
		{
			if (NewProjectItem->ProjectFile == LastSelectedProjectFile)
			{
				NewProjectToSelect = NewProjectItem;
			}

			// Get a valid last access time. The editor will set this for recent projects and is more accurate than the NTFS access time
			if (FRecentProjectFile* RecentProject = RecentProjects.FindByKey(NewProjectItem->ProjectFile))
			{
				NewProjectItem->LastAccessTime = RecentProject->LastOpenTime;
			}

			ProjectItemsSource.Add(NewProjectItem);
		}
	}

	SortProjectTiles(CurrentSortOption);

	PopulateFilteredProjects();

	if (NewProjectToSelect && FilteredProjectItemsSource.Contains(NewProjectToSelect))
	{
		ProjectTileView->SetSelection(NewProjectToSelect, ESelectInfo::Direct);
	}
	else if (FilteredProjectItemsSource.Num() > 0)
	{
		ProjectTileView->SetSelection(FilteredProjectItemsSource[0], ESelectInfo::Direct);
	}

	return FReply::Handled();
}

void SProjectBrowser::PopulateFilteredProjects()
{
	FilteredProjectItemsSource.Empty();

	for (TSharedPtr<FProjectItem>& ProjectItem : ProjectItemsSource)
	{
		if (ProjectItemFilter.PassesFilter(ProjectItem))
		{
			FilteredProjectItemsSource.Add(ProjectItem);
		}
	}

	ProjectTileView->RequestListRefresh();
}

FReply SProjectBrowser::OnKeyDown( const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent )
{
	if (InKeyEvent.GetKey() == EKeys::F5)
	{
		return FindProjects();
	}

	return FReply::Unhandled();
}

bool SProjectBrowser::OpenProject( const FString& InProjectFile )
{
	FText FailReason;
	FString ProjectFile = InProjectFile;

	// Get the identifier for the project
	FString ProjectIdentifier;
	FDesktopPlatformModule::Get()->GetEngineIdentifierForProject(ProjectFile, ProjectIdentifier);

	// Abort straight away if the project engine version is newer than the current engine version
	FEngineVersion EngineVersion;
	if (FDesktopPlatformModule::Get()->TryParseStockEngineVersion(ProjectIdentifier, EngineVersion))
	{
		if (FEngineVersion::GetNewest(EngineVersion, FEngineVersion::Current(), nullptr) == EVersionComparison::First)
		{
			FMessageDialog::Open(EAppMsgType::Ok, LOCTEXT("CantLoadNewerProject", "Unable to open this project, as it was made with a newer version of the Unreal Engine."));
			return false;
		}
	}
	
	// Get the identifier for the current engine
	FString CurrentIdentifier = FDesktopPlatformModule::Get()->GetCurrentEngineIdentifier();
	if(ProjectIdentifier != CurrentIdentifier)
	{
		// Get the current project status
		FProjectStatus ProjectStatus;
		if(!IProjectManager::Get().QueryStatusForProject(ProjectFile, ProjectStatus))
		{
			FMessageDialog::Open(EAppMsgType::Ok, LOCTEXT("CouldNotReadProjectStatus", "Unable to read project status."));
			return false;
		}

		// If it's a code project, verify the user has the needed compiler installed before we continue.
		if ( ProjectStatus.bCodeBasedProject )
		{
			if ( !FSourceCodeNavigation::IsCompilerAvailable() )
			{
				const FText TitleText = LOCTEXT("CompilerNeeded", "Missing Compiler");
				const FText CompilerStillNotInstalled = FText::Format(LOCTEXT("CompilerStillNotInstalledFormatted", "Press OK when you've finished installing {0}."), FSourceCodeNavigation::GetSuggestedSourceCodeIDE());

				if ( FSourceCodeNavigation::GetCanDirectlyInstallSourceCodeIDE() )
				{
					const FText ErrorText = FText::Format(LOCTEXT("WouldYouLikeToDownloadAndInstallCompiler", "To open this project you must first install {0}.\n\nWould you like to download and install it now?"), FSourceCodeNavigation::GetSuggestedSourceCodeIDE());

					EAppReturnType::Type InstallCompilerResult = FMessageDialog::Open(EAppMsgType::YesNo, ErrorText, TitleText);
					if ( InstallCompilerResult == EAppReturnType::No )
					{
						return false;
					}

					GWarn->BeginSlowTask(LOCTEXT("DownloadingInstalling", "Waiting for Installer to complete."), true, true);

					TOptional<bool> bWasDownloadASuccess;

					FSourceCodeNavigation::DownloadAndInstallSuggestedIDE(FOnIDEInstallerDownloadComplete::CreateLambda([&bWasDownloadASuccess] (bool bSuccessful) {
						bWasDownloadASuccess = bSuccessful;
					}));

					while ( !bWasDownloadASuccess.IsSet() )
					{
						// User canceled the install.
						if ( GWarn->ReceivedUserCancel() )
						{
							GWarn->EndSlowTask();
							return false;
						}

						GWarn->StatusUpdate(1, 1, LOCTEXT("WaitingForDownload", "Waiting for download to complete..."));
						FPlatformProcess::Sleep(0.1f);
					}

					GWarn->EndSlowTask();

					if ( !bWasDownloadASuccess.GetValue() )
					{
						const FText DownloadFailed = LOCTEXT("DownloadFailed", "Failed to download. Please check your internet connection.");
						if ( FMessageDialog::Open(EAppMsgType::OkCancel, DownloadFailed) == EAppReturnType::Cancel )
						{
							// User canceled, fail.
							return false;
						}
					}
				}
				else
				{
					const FText ErrorText = FText::Format(LOCTEXT("WouldYouLikeToInstallCompiler", "To open this project you must first install {0}.\n\nWould you like to install it now?"), FSourceCodeNavigation::GetSuggestedSourceCodeIDE());
					EAppReturnType::Type InstallCompilerResult = FMessageDialog::Open(EAppMsgType::YesNo, ErrorText, TitleText);
					if ( InstallCompilerResult == EAppReturnType::No )
					{
						return false;
					}

					FString DownloadURL = FSourceCodeNavigation::GetSuggestedSourceCodeIDEDownloadURL();
					FPlatformProcess::LaunchURL(*DownloadURL, nullptr, nullptr);
				}

				// Loop until the users cancels or they complete installation.
				while ( !FSourceCodeNavigation::IsCompilerAvailable() )
				{
					EAppReturnType::Type UserInstalledResult = FMessageDialog::Open(EAppMsgType::OkCancel, CompilerStillNotInstalled);
					if ( UserInstalledResult == EAppReturnType::Cancel )
					{
						return false;
					}

					FSourceCodeNavigation::RefreshCompilerAvailability();
				}
			}
		}

		// Hyperlinks for the upgrade dialog
		TArray<FText> Hyperlinks;
		int32 MoreOptionsHyperlink = Hyperlinks.Add(LOCTEXT("ProjectConvert_MoreOptions", "More Options..."));

		// Button labels for the upgrade dialog
		TArray<FText> Buttons;
		int32 OpenCopyButton = Buttons.Add(LOCTEXT("ProjectConvert_OpenCopy", "Open a copy"));
		int32 CancelButton = Buttons.Add(LOCTEXT("ProjectConvert_Cancel", "Cancel"));
		int32 OpenExistingButton = -1;
		int32 SkipConversionButton = -1;

		// Prompt for upgrading. Different message for code and content projects, since the process is a bit trickier for code.
		FText DialogText;
		if(ProjectStatus.bCodeBasedProject)
		{
			DialogText = LOCTEXT("ConvertCodeProjectPrompt", "This project was made with a different version of the Unreal Engine. Converting to this version will rebuild your code projects.\n\nNew features and improvements sometimes cause API changes, which may require you to modify your code before it compiles. Content saved with newer versions of the editor will not open in older versions.\n\nWe recommend you open a copy of your project to avoid damaging the original.");
		}
		else
		{
			DialogText = LOCTEXT("ConvertContentProjectPrompt", "This project was made with a different version of the Unreal Engine.\n\nOpening it with this version of the editor may prevent it opening with the original editor, and may lose data. We recommend you open a copy to avoid damaging the original.");
		}

		// Show the dialog, and expand to the advanced dialog if the user selects 'More Options...'
		int32 Selection = SVerbChoiceDialog::ShowModal(LOCTEXT("ProjectConversionTitle", "Convert Project"), DialogText, Hyperlinks, Buttons);
		if(~Selection == MoreOptionsHyperlink)
		{
			OpenExistingButton = Buttons.Insert(LOCTEXT("ProjectConvert_ConvertInPlace", "Convert in-place"), 1);
			SkipConversionButton = Buttons.Insert(LOCTEXT("ProjectConvert_SkipConversion", "Skip conversion"), 2);
			CancelButton += 2;
			Selection = SVerbChoiceDialog::ShowModal(LOCTEXT("ProjectConversionTitle", "Convert Project"), DialogText, Buttons);
		}

		// Handle the selection
		if(Selection == CancelButton)
		{
			return false;
		}
		if(Selection == OpenCopyButton)
		{
			FString NewProjectFile;
			GameProjectUtils::EProjectDuplicateResult DuplicateResult = GameProjectUtils::DuplicateProjectForUpgrade(ProjectFile, NewProjectFile);

			if (DuplicateResult == GameProjectUtils::EProjectDuplicateResult::UserCanceled)
			{
				return false;
			}
			else if (DuplicateResult == GameProjectUtils::EProjectDuplicateResult::Failed)
			{
				FMessageDialog::Open( EAppMsgType::Ok, LOCTEXT("ConvertProjectCopyFailed", "Couldn't copy project. Check you have sufficient hard drive space and write access to the project folder.") );
				return false;
			}

			ProjectFile = NewProjectFile;
		}
		if(Selection == OpenExistingButton)
		{
			FString FailPath;
			if(!FDesktopPlatformModule::Get()->CleanGameProject(FPaths::GetPath(ProjectFile), FailPath, GWarn))
			{
				FText FailMessage = FText::Format(LOCTEXT("ConvertProjectCleanFailed", "{0} could not be removed. Try deleting it manually and try again."), FText::FromString(FailPath));
				FMessageDialog::Open(EAppMsgType::Ok, FailMessage);
				return false;
			}
		}
		if(Selection != SkipConversionButton)
		{
			// Update the game project to the latest version. This will prompt to check-out as necessary. We don't need to write the engine identifier directly, because it won't use the right .uprojectdirs logic.
			if(!GameProjectUtils::UpdateGameProject(ProjectFile, CurrentIdentifier, FailReason))
			{
				if(FMessageDialog::Open(EAppMsgType::YesNo, LOCTEXT("ProjectUpgradeFailure", "The project file could not be updated to latest version. Attempt to open anyway?")) != EAppReturnType::Yes)
				{
					return false;
				}
			}

			// If it's a code-based project, generate project files and open visual studio after an upgrade
			if(ProjectStatus.bCodeBasedProject)
			{
				// Try to generate project files
				FStringOutputDevice OutputLog;
				OutputLog.SetAutoEmitLineTerminator(true);
				GLog->AddOutputDevice(&OutputLog);
				bool bHaveProjectFiles = FDesktopPlatformModule::Get()->GenerateProjectFiles(FPaths::RootDir(), ProjectFile, GWarn);
				GLog->RemoveOutputDevice(&OutputLog);

				// Display any errors
				if(!bHaveProjectFiles)
				{
					FFormatNamedArguments Args;
					Args.Add( TEXT("LogOutput"), FText::FromString(OutputLog) );
					FMessageDialog::Open(EAppMsgType::Ok, FText::Format(LOCTEXT("CouldNotGenerateProjectFiles", "Project files could not be generated. Log output:\n\n{LogOutput}"), Args));
					return false;
				}

				// Try to compile the project
				if(!GameProjectUtils::BuildCodeProject(ProjectFile))
				{
					return false;
				}
			}
		}
	}

	// Open the project
	if (!GameProjectUtils::OpenProject(ProjectFile, FailReason))
	{
		FMessageDialog::Open(EAppMsgType::Ok, FailReason);
		return false;
	}

	return true;
}


void SProjectBrowser::OpenSelectedProject( )
{
	if (CurrentSelectedProjectPath.IsEmpty())
	{
		return;
	}

	OpenProject(CurrentSelectedProjectPath);
}

/* SProjectBrowser event handlers
 *****************************************************************************/

FReply SProjectBrowser::OnOpenProject( )
{
	OpenSelectedProject();

	return FReply::Handled();
}


bool SProjectBrowser::HandleOpenProjectButtonIsEnabled( ) const
{
	return !CurrentSelectedProjectPath.IsEmpty();
}


void SProjectBrowser::HandleProjectItemDoubleClick( TSharedPtr<FProjectItem> TemplateItem )
{
	OpenSelectedProject();
}

FReply SProjectBrowser::OnBrowseToProject()
{
	const FString ProjectFileDescription = LOCTEXT( "FileTypeDescription", "Unreal Project File" ).ToString();
	const FString ProjectFileExtension = FString::Printf(TEXT("*.%s"), *FProjectDescriptor::GetExtension());
	const FString FileTypes = FString::Printf( TEXT("%s (%s)|%s"), *ProjectFileDescription, *ProjectFileExtension, *ProjectFileExtension );

	// Find the first valid project file to select by default
	FString DefaultFolder = FEditorDirectories::Get().GetLastDirectory(ELastDirectory::PROJECT);
	for (const FRecentProjectFile& RecentFile : GetDefault<UEditorSettings>()->RecentlyOpenedProjectFiles)
	{
		if ( IFileManager::Get().FileSize(*RecentFile.ProjectName) > 0 )
		{
			// This is the first uproject file in the recents list that actually exists
			DefaultFolder = FPaths::GetPath(*RecentFile.ProjectName);
			break;
		}
	}

	// Prompt the user for the filenames
	TArray<FString> OpenFilenames;
	IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
	bool bOpened = false;
	if ( DesktopPlatform )
	{
		void* ParentWindowWindowHandle = NULL;

		IMainFrameModule& MainFrameModule = FModuleManager::LoadModuleChecked<IMainFrameModule>(TEXT("MainFrame"));
		const TSharedPtr<SWindow>& MainFrameParentWindow = MainFrameModule.GetParentWindow();
		if ( MainFrameParentWindow.IsValid() && MainFrameParentWindow->GetNativeWindow().IsValid() )
		{
			ParentWindowWindowHandle = MainFrameParentWindow->GetNativeWindow()->GetOSWindowHandle();
		}

		bOpened = DesktopPlatform->OpenFileDialog(
			ParentWindowWindowHandle,
			LOCTEXT("OpenProjectBrowseTitle", "Open Project").ToString(),
			DefaultFolder,
			TEXT(""),
			FileTypes,
			EFileDialogFlags::None,
			OpenFilenames
		);
	}

	if (bOpened && OpenFilenames.Num() > 0)
	{
		HandleProjectViewSelectionChanged(nullptr, ESelectInfo::Direct);

		FString Path = OpenFilenames[0];
		if ( FPaths::IsRelative( Path ) )
		{
			Path = FPaths::ConvertRelativePathToFull(Path);
		}

		CurrentSelectedProjectPath = Path;

		OpenSelectedProject();
	}

	return FReply::Handled();
}

void SProjectBrowser::HandleProjectViewSelectionChanged(TSharedPtr<FProjectItem> ProjectItem, ESelectInfo::Type SelectInfo)
{
	FString ProjectFile;
	if (ProjectItem.IsValid())
	{
		ProjectFile = ProjectItem->ProjectFile;
		CurrentSelectedProjectPath = ProjectFile;
	}
	else
	{
		CurrentSelectedProjectPath = FString();
	}

	ProjectSelectionChangedDelegate.ExecuteIfBound(ProjectFile);
}

void SProjectBrowser::OnFilterTextChanged(const FText& InText)
{
	ProjectItemFilter.SetRawFilterText(InText);
	SearchBoxPtr->SetError(ProjectItemFilter.GetFilterErrorText());
	PopulateFilteredProjects();
}

void SProjectBrowser::OnAutoloadLastProjectChanged(ECheckBoxState NewState)
{
	UEditorSettings *Settings = GetMutableDefault<UEditorSettings>();

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	Settings->bLoadTheMostRecentlyLoadedProjectAtStartup = (NewState == ECheckBoxState::Checked);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	FProperty* AutoloadProjectProperty = FindFProperty<FProperty>(Settings->GetClass(), "bLoadTheMostRecentlyLoadedProjectAtStartup");
	if (AutoloadProjectProperty != NULL)
	{
		FPropertyChangedEvent PropertyUpdateStruct(AutoloadProjectProperty);
		Settings->PostEditChangeProperty(PropertyUpdateStruct);
	}
}

EVisibility SProjectBrowser::GetProjectCategoryVisibility(const TSharedRef<FProjectCategory> InCategory) const
{
	return FilteredProjectItemsSource.Num() ? EVisibility::Visible : EVisibility::Collapsed;
}

EVisibility SProjectBrowser::GetNoProjectsErrorVisibility() const
{
	return ProjectItemsSource.Num() > 0 ? EVisibility::Collapsed : EVisibility::Visible;
}

EVisibility SProjectBrowser::GetNoProjectsAfterFilterErrorVisibility() const
{
	return (ProjectItemsSource.Num() && FilteredProjectItemsSource.Num() == 0) ? EVisibility::Visible : EVisibility::Collapsed;
}

void SProjectBrowser::SortProjectTiles(EProjectSortOption NewSortOption)
{
	CurrentSortOption = NewSortOption;

	switch (CurrentSortOption)
	{
	case EProjectSortOption::Version:
		ProjectItemsSource.StableSort([](auto& A, auto& B) { return A->CompareEngineVersion(*B); });
		break;
	case EProjectSortOption::Alphabetical:
		ProjectItemsSource.StableSort([](auto& A, auto& B) { return A->Name.CompareTo(B->Name) < 0; });
		break;
	case EProjectSortOption::LastAccessTime:
	default:
		ProjectItemsSource.StableSort([](auto& A, auto& B) { return A->LastAccessTime > B->LastAccessTime; });
		break;
	}

	PopulateFilteredProjects();
}

ECheckBoxState SProjectBrowser::GetSortOptionCheckState(EProjectSortOption TestOption) const
{
	return CurrentSortOption == TestOption ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

FText SProjectBrowser:: GetItemHighlightText() const
{
	return ProjectItemFilter.GetRawFilterText();
}

#undef LOCTEXT_NAMESPACE
