// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/BitArray.h"
#include "Containers/Set.h"
#include "Containers/SparseArray.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "Input/Reply.h"
#include "Internationalization/Text.h"
#include "Layout/Visibility.h"
#include "Misc/Optional.h"
#include "Misc/TextFilter.h"
#include "Templates/SharedPointer.h"
#include "Templates/TypeHash.h"
#include "Templates/UnrealTemplate.h"
#include "Types/SlateConstants.h"
#include "Types/SlateEnums.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STileView.h"

class ITableRow;
class SToolTip;
class SVerticalBox;
class SWidget;
struct FGeometry;
struct FKeyEvent;
struct FProjectCategory;
struct FProjectItem;

enum class EProjectSortOption : uint8
{
	/** Sort by the last time the project was opened */
	LastAccessTime,

	/** Sort by engine version */
	Version,

	/** By project name */
	Alphabetical,
};


enum class ECheckBoxState : uint8;

DECLARE_DELEGATE_OneParam(FProjectSelectionChanged, FString);

/**
 * A list of known projects with the option to add a new one
 */
class SProjectBrowser
	: public SCompoundWidget
{
public:

	DECLARE_DELEGATE(FNewProjectScreenRequested)

	SLATE_BEGIN_ARGS(SProjectBrowser)
		: _ShowHeaderBar(true)
		, _AllowScrolling(true)
		, _IsEmbedded(false)
	{}
		SLATE_ARGUMENT(bool, ShowHeaderBar)
		SLATE_ARGUMENT(bool, AllowScrolling)

		/** If true it will show the embedded version of the ProjectBrowser */
		SLATE_ARGUMENT(bool, IsEmbedded)
		SLATE_ARGUMENT(FProjectSelectionChanged, OnSelectionChanged);
	SLATE_END_ARGS()

public:

	/** Constructor */
	SProjectBrowser();

	/**
	 * Constructs this widget with InArgs.
	 *
	 * @param InArgs - The construction arguments.
	 */
	void Construct(const FArguments& InArgs);

	bool HasProjects() const;

	bool HasSelectedProjectFile() const { return !GetSelectedProjectFile().IsEmpty(); }

	FString GetSelectedProjectFile() const;

	FText GetSelectedProjectName() const;

	/** Begins the opening process for the selected project */
	void OpenSelectedProject();

	FReply OnBrowseToProject();

	/** Handler for when the Open button is clicked */
	FReply OnOpenProject();

	/** Whether to autoload the last project. */
	void OnAutoloadLastProjectChanged(ECheckBoxState NewState);
protected:

	/** Creates a row in the template list */
	TSharedRef<ITableRow> MakeProjectViewWidget(TSharedPtr<FProjectItem> ProjectItem, const TSharedRef<STableViewBase>& OwnerTable);

	/** Create a tooltip for the given project item */
	TSharedRef<SToolTip> MakeProjectToolTip(TSharedPtr<FProjectItem> ProjectItem) const;

	/** Add information to the tooltip for this project item */
	void AddToToolTipInfoBox(const TSharedRef<SVerticalBox>& InfoBox, const FText& Key, const FText& Value) const;

	/** Get the context menu to use for the selected project item */
	TSharedPtr<SWidget> OnGetContextMenuContent() const;

	/** Handler for when find in explorer is selected */
	void ExecuteFindInExplorer() const;

	/** Handler to check to see if a find in explorer command is allowed */
	bool CanExecuteFindInExplorer() const;

	/** Gets the currently selected template item */
	TSharedPtr<FProjectItem> GetSelectedProjectItem() const;

	/** Populates ProjectItemsSource with projects found on disk */
	FReply FindProjects();

	/** Opens the specified project file */
	bool OpenProject(const FString& ProjectFile);

	/** Populate the list of filtered project categories */
	void PopulateFilteredProjects();

	/**
	 * Called after a key is pressed when this widget has focus (this event bubbles if not handled)
	 *
	 * @param MyGeometry The Geometry of the widget receiving the event
	 * @param  InKeyEvent  Key event
	 *
	 * @return  Returns whether the event was handled, along with other possible actions
	 */
	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;

private:
	/** Returns true if the user is allowed to open a project with the supplied name and path */
	bool HandleOpenProjectButtonIsEnabled() const;

	/** Called when a user double-clicks a project item in the project view */
	void HandleProjectItemDoubleClick(TSharedPtr<FProjectItem> ProjectItem);

	/** Handler for when the selection changes in the project view */
	void HandleProjectViewSelectionChanged(TSharedPtr<FProjectItem> ProjectItem, ESelectInfo::Type SelectInfo);

	/** Called when the text in the filter box is changed */
	void OnFilterTextChanged(const FText& InText);

	/** Get the visibility of the specified category */
	EVisibility GetProjectCategoryVisibility(const TSharedRef<FProjectCategory> InCategory) const;

	EVisibility GetNoProjectsErrorVisibility() const;

	/** Get the visibility of the "no projects" error */
	EVisibility GetNoProjectsAfterFilterErrorVisibility() const;

	/** Get the filter text to highlight on items in the list */
	FText GetItemHighlightText() const;

	void SortProjectTiles(EProjectSortOption NewSortOption);
	ECheckBoxState GetSortOptionCheckState(EProjectSortOption TestOption) const;

private:
	TSharedPtr<STileView<TSharedPtr<FProjectItem>>> ProjectTileView;
	TArray<TSharedPtr<FProjectItem>> ProjectItemsSource;
	TArray<TSharedPtr<FProjectItem>> FilteredProjectItemsSource;
	TSharedPtr<SVerticalBox> ProjectsBox;

	FProjectSelectionChanged ProjectSelectionChangedDelegate;

	/** Search box used to set the filter text */
	TSharedPtr<class SSearchBox> SearchBoxPtr;

	/** Filter that is used to test for the visibility of projects */
	typedef TTextFilter<const TSharedPtr<FProjectItem>> ProjectItemTextFilter;
	ProjectItemTextFilter ProjectItemFilter;

	bool bPreventSelectionChangeEvent;

	FString CurrentSelectedProjectPath;

	bool IsOnlineContentFinished;

	/** If true it will show the embedded version of the ProjectBrowser */
	bool bIsEmbedded;

	// Holds a delegate that is executed when the new project screen is being requested.
	FNewProjectScreenRequested NewProjectScreenRequestedDelegate;

	EProjectSortOption CurrentSortOption = EProjectSortOption::LastAccessTime;
};

