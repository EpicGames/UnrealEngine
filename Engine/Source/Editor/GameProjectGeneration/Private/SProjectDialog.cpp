// Copyright Epic Games, Inc. All Rights Reserved.

#include "SProjectDialog.h"
#include "Algo/Transform.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/SBoxPanel.h"
#include "GameProjectGenerationModule.h"
#include "TemplateCategory.h"
#include "SProjectBrowser.h"
#include "Widgets/Layout/SSeparator.h"
#include "IDocumentation.h"
#include "IDesktopPlatform.h"
#include "DesktopPlatformModule.h"
#include "TemplateItem.h"
#include "SourceCodeNavigation.h"
#include "GameProjectUtils.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/Commands/UIAction.h"
#include "Editor.h"
#include "Internationalization/BreakIterator.h"
#include "Settings/EditorSettings.h"
#include "Widgets/Layout/SScrollBorder.h"
#include "Widgets/Layout/SScrollBox.h"
#include "HardwareTargetingModule.h"
#include "Widgets/Input/SSegmentedControl.h"
#include "GameProjectGenerationLog.h"
#include "Interfaces/IPluginManager.h"
#include "HAL/PlatformFileManager.h"
#include "Dialogs/SOutputLogDialog.h"
#include "Misc/MessageDialog.h"
#include "HAL/FileManager.h"
#include "ProjectDescriptor.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/SToolTip.h"
#include "Widgets/Layout/SGridPanel.h"
#include "Widgets/SOverlay.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SButton.h"
#include "SWarningOrErrorBox.h"
#include "SGetSuggestedIDEWidget.h"
#include "Brushes/SlateDynamicImageBrush.h"
#include "Widgets/Layout/SWidgetSwitcher.h"
#include "SProjectDialog.h"
#include "LauncherPlatformModule.h"
#include "SPrimaryButton.h"
#include "Styling/StyleColors.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateStyleRegistry.h"

#define LOCTEXT_NAMESPACE "GameProjectGeneration"

namespace NewProjectDialogDefs
{
	constexpr float TemplateTileHeight = 153;
	constexpr float TemplateTileWidth = 102;
	constexpr float ThumbnailSize = 64.0f, ThumbnailPadding = 5.f;
	const FName DefaultCategoryName = "Games";
	const FName BlankCategoryKey = "Default";
}

TSharedPtr<FSlateBrush> SProjectDialog::CustomTemplateBrush;

class SMajorCategoryTile : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SMajorCategoryTile) {}
		SLATE_ATTRIBUTE(bool, IsSelected)
	SLATE_END_ARGS()

public:
	void Construct(const FArguments& InArgs, TSharedPtr<FTemplateCategory> Item)
	{
		IsSelected = InArgs._IsSelected;
		TemplateCategoryWeak = Item;

		ChildSlot
		[
			SNew(SOverlay)
			+ SOverlay::Slot()
			[
				SNew(SImage)
				.Image(this, &SMajorCategoryTile::GetIconBrush)
			]
			+ SOverlay::Slot()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Bottom)
			.Padding(FMargin(18.0f, 8.0f))
			[
				SNew(STextBlock)
				.Font(FAppStyle::Get().GetFontStyle("HeadingExtraSmall"))
				.ColorAndOpacity(FLinearColor(1, 1, 1, .9f))
				.TransformPolicy(ETextTransformPolicy::ToUpper)
				.ShadowOffset(FVector2D(1, 1))
				.ShadowColorAndOpacity(FLinearColor(0, 0, 0, .75))
				.Text_Lambda([WeakItem = Item.ToWeakPtr()]()
				{
					if (const TSharedPtr<FTemplateCategory> TemplateCategory = WeakItem.Pin())
					{
						return TemplateCategory->GetDisplayName();
					}
					return FText::GetEmpty();
				})
				.WrapTextAt(250.0f)
			]
			+ SOverlay::Slot()
			[
				SNew(SImage)
				.Visibility(EVisibility::HitTestInvisible)
				.Image(this, &SMajorCategoryTile::GetSelectionOutlineBrush)
			]
		];
	}

private:
	const FSlateBrush* GetSelectionOutlineBrush() const
	{
		const bool bIsSelected = IsSelected.Get();
		const bool bIsTileHovered = IsHovered();

		if (bIsSelected && bIsTileHovered)
		{
			static const FName SelectedHover("ProjectBrowser.ProjectTile.SelectedHoverBorder");
			return FAppStyle::Get().GetBrush(SelectedHover);
		}
		else if (bIsSelected)
		{
			static const FName Selected("ProjectBrowser.ProjectTile.SelectedBorder");
			return FAppStyle::Get().GetBrush(Selected);
		}
		else if (bIsTileHovered)
		{
			static const FName Hovered("ProjectBrowser.ProjectTile.HoverBorder");
			return FAppStyle::Get().GetBrush(Hovered);
		}

		return FStyleDefaults::GetNoBrush();
	}

	const FSlateBrush* GetIconBrush() const
	{
		if (TSharedPtr<FTemplateCategory> TemplateCategory = TemplateCategoryWeak.Pin())
		{
			if (TSharedPtr<FSlateBrush> Icon = TemplateCategory->Icon; Icon && FPaths::FileExists(Icon->GetResourceName().ToString()))
			{
				return Icon.Get();
			}
		}

		return FAppStyle::Get().GetBrush("ProjectBrowser.MajorCategory.FallBackThumbnail");
	}

private:
	TAttribute<bool> IsSelected;
	TWeakPtr<FTemplateCategory> TemplateCategoryWeak;
};


/** Slate tile widget for template projects */
class STemplateTile : public STableRow<TSharedPtr<FTemplateItem>>
{
public:
	SLATE_BEGIN_ARGS( STemplateTile ){}
		SLATE_ARGUMENT(TSharedPtr<FTemplateItem>, Item)
	SLATE_END_ARGS()

private:
	TWeakPtr<FTemplateItem> Item;

public:
	/** Static build function */
	static TSharedRef<ITableRow> BuildTile(TSharedPtr<FTemplateItem> Item, const TSharedRef<STableViewBase>& OwnerTable)
	{
		if (!ensure(Item.IsValid()))
		{
			return SNew(STableRow<TSharedPtr<FTemplateItem>>, OwnerTable);
		}

		return SNew(STemplateTile, OwnerTable).Item(Item);
	}

	/** Constructs this widget with InArgs */
	void Construct( const FArguments& InArgs, const TSharedRef<STableViewBase>& OwnerTable )
	{
		check(InArgs._Item.IsValid())
		Item = InArgs._Item;

		STableRow::FArguments TableRowArguments;
		TableRowArguments._SignalSelectionMode = ETableRowSignalSelectionMode::Instantaneous;

		TSharedPtr<SOverlay> ProjectTileOverlay;

		STableRow::Construct(
			TableRowArguments
			.Style(FAppStyle::Get(), "ProjectBrowser.TableRow")
			.Padding(2.0f)
			.Content()
			[
				SNew(SBorder)
				.Padding(FMargin(0.0f, 0.0f, 5.0f, 5.0f))
				.BorderImage(FAppStyle::Get().GetBrush("ProjectBrowser.ProjectTile.DropShadow"))
				[
					SAssignNew(ProjectTileOverlay, SOverlay)
					+ SOverlay::Slot()
					[
						SNew(SVerticalBox)
						// Thumbnail
						+ SVerticalBox::Slot()
						.AutoHeight()
						.HAlign(HAlign_Center)
						.VAlign(VAlign_Center)
						[
							SNew(SBox)
							.WidthOverride(102.0f)
							.HeightOverride(102.0f)
							[
								SNew(SBorder)
								.Padding(0.0f)
								.BorderImage(FAppStyle::Get().GetBrush("ProjectBrowser.ProjectTile.ThumbnailAreaBackground"))
								.HAlign(HAlign_Center)
								.VAlign(VAlign_Center)
								[
									SNew(SImage)
									.Image(this, &STemplateTile::GetThumbnail)
									.DesiredSizeOverride(InArgs._Item->bThumbnailAsIcon ? TOptional<FVector2D>() : FVector2D(NewProjectDialogDefs::ThumbnailSize, NewProjectDialogDefs::ThumbnailSize))
								]
							]
						]
						// Name
						+ SVerticalBox::Slot()
						[
							SNew(SBorder)
							.Padding(FMargin(NewProjectDialogDefs::ThumbnailPadding, 0))
							.VAlign(VAlign_Top)
							.Padding(FMargin(3.0f, 3.0f))
							.BorderImage_Lambda
							(
								[this]()
								{
									const bool bIsSelected = IsSelected();
									const bool bIsRowHovered = IsHovered();

									if (bIsSelected && bIsRowHovered)
									{
										static const FName SelectedHover("ProjectBrowser.ProjectTile.NameAreaSelectedHoverBackground");
										return FAppStyle::Get().GetBrush(SelectedHover);
									}
									else if (bIsSelected)
									{
										static const FName Selected("ProjectBrowser.ProjectTile.NameAreaSelectedBackground");
										return FAppStyle::Get().GetBrush(Selected);
									}
									else if (bIsRowHovered)
									{
										static const FName Hovered("ProjectBrowser.ProjectTile.NameAreaHoverBackground");
										return FAppStyle::Get().GetBrush(Hovered);
									}

									return FAppStyle::Get().GetBrush("ProjectBrowser.ProjectTile.NameAreaBackground");
								}
							)
							[
								SNew(STextBlock)
								.Font(FAppStyle::Get().GetFontStyle("ProjectBrowser.ProjectTile.Font"))
								.WrapTextAt(NewProjectDialogDefs::TemplateTileWidth-4.0f)
								.LineBreakPolicy(FBreakIterator::CreateCamelCaseBreakIterator())
								.Text_Lambda([this]()
								{
									if (const TSharedPtr<FTemplateItem> ItemPtr = Item.Pin())
									{
										return ItemPtr->GetDisplayName();
									}
									return FText::GetEmpty();
								})
								.ColorAndOpacity_Lambda
								(
									[this]()
									{
										const bool bIsSelected = IsSelected();
										const bool bIsRowHovered = IsHovered();

										if (bIsSelected || bIsRowHovered)
										{
											return FStyleColors::White;
										}

										return FSlateColor::UseForeground();
									}
								)
							]
						]
					]
					+ SOverlay::Slot()
					[
						SNew(SImage)
						.Visibility(EVisibility::HitTestInvisible)
						.Image_Lambda
						(
							[this]()
							{
								const bool bIsSelected = IsSelected();
								const bool bIsRowHovered = IsHovered();

								if (bIsSelected && bIsRowHovered)
								{
									static const FName SelectedHover("ProjectBrowser.ProjectTile.SelectedHoverBorder");
									return FAppStyle::Get().GetBrush(SelectedHover);
								}
								else if (bIsSelected)
								{
									static const FName Selected("ProjectBrowser.ProjectTile.SelectedBorder");
									return FAppStyle::Get().GetBrush(Selected);
								}
								else if (bIsRowHovered)
								{
									static const FName Hovered("ProjectBrowser.ProjectTile.HoverBorder");
									return FAppStyle::Get().GetBrush(Hovered);
								}

								return FStyleDefaults::GetNoBrush();
							}
						)
					]
				]
			],
			OwnerTable
		);

		if (const TSharedPtr<IPlugin> Plugin = InArgs._Item->PluginWeak.Pin(); Plugin && !Plugin->IsEnabled())
		{
			if (ProjectTileOverlay.IsValid())
			{
				ProjectTileOverlay->AddSlot()
				.HAlign(HAlign_Right)
				.VAlign(VAlign_Bottom)
				.Padding(2)
				[
					SNew(SImage)
						.Image(FAppStyle::GetBrush("Icons.Warning"))
						.ToolTipText(LOCTEXT("WarnNotEnabled", "This template comes from a plugin that is not enabled by default."))
						.ColorAndOpacity(EStyleColor::Warning)
				];
			}
		}
	}

private:

	/** Get this item's thumbnail or return the default */
	const FSlateBrush* GetThumbnail() const
	{
		TSharedPtr<FTemplateItem> ItemPtr = Item.Pin();
		if (ItemPtr.IsValid() && ItemPtr->Thumbnail.IsValid())
		{
			return ItemPtr->Thumbnail.Get();
		}
		return FAppStyle::GetBrush("UnrealDefaultThumbnailRounded");
	}
	
};

/**
 * Simple widget used to display a folder path, and a name of a file
 */
class SFilepath : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS( SFilepath )
		: _IsReadOnly(false)
	{}
		/** Attribute specifying the text to display in the folder input */
		SLATE_ATTRIBUTE(FText, FolderPath)

		/** Attribute specifying the text to display in the name input */
		SLATE_ATTRIBUTE(FText, Name)

		SLATE_ATTRIBUTE(FText, WarningText)

		SLATE_ARGUMENT(bool, IsReadOnly)

		/** Event that is triggered when the browser for folder button is clicked */
		SLATE_EVENT(FOnClicked, OnBrowseForFolder)

		/** Events for when the name field is manipulated */
		SLATE_EVENT(FOnTextChanged, OnNameChanged)
		SLATE_EVENT(FOnTextCommitted, OnNameCommitted)
		
		/** Events for when the folder field is manipulated */
		SLATE_EVENT(FOnTextChanged, OnFolderChanged)
		SLATE_EVENT(FOnTextCommitted, OnFolderCommitted)

	SLATE_END_ARGS()

	/** Constructs this widget with InArgs */
	void Construct( const FArguments& InArgs )
	{
		WarningText = InArgs._WarningText;

		ChildSlot
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Top)
			.Padding(0.0f, 4.0f, 8.0f, 8.0f)
			.AutoWidth()
			[
				SNew(STextBlock)
				.Text(LOCTEXT("ProjectLocation", "Project Location"))
			]
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Top)
			.AutoWidth()
			[
				SNew(SBox)
				.WidthOverride(595.0f)
				[
					SNew(SVerticalBox)
					+ SVerticalBox::Slot()
					.AutoHeight()
					[
						SNew(SEditableTextBox)
						.IsReadOnly(InArgs._IsReadOnly)
						.Text(InArgs._FolderPath)
						.OnTextChanged(InArgs._OnFolderChanged)
						.OnTextCommitted(InArgs._OnFolderCommitted)
					]
					+ SVerticalBox::Slot()
					.Padding(0.0f, 8.0f)
					[	
						SNew(SWarningOrErrorBox)
						.Visibility(this, &SFilepath::GetWarningVisibility)
						.IconSize(FVector2D(16,16))
						.Padding(FMargin(8.0f, 4.0f, 4.0f, 4.0f))
						.Message(WarningText)
						.MessageStyle(EMessageStyle::Error)
					]
				]
			]
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Top)
			.Padding(2.0f, 2.0f, 0.0f, 0.0f)
			.AutoWidth()
			[
				SNew(SButton)
				.ButtonStyle(FAppStyle::Get(), "SimpleButton")
				.OnClicked(InArgs._OnBrowseForFolder)
				.ToolTipText(LOCTEXT("BrowseForFolder", "Browse for a folder"))
				.Visibility(InArgs._IsReadOnly ? EVisibility::Hidden : EVisibility::Visible)
				.ContentPadding(0.0f)
				[
					SNew(SImage)
					.Image(FAppStyle::Get().GetBrush("Icons.FolderClosed"))
					.ColorAndOpacity(FSlateColor::UseForeground())
				]
			]
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Right)
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.VAlign(VAlign_Top)
				.AutoWidth()
				.Padding(32.0f, 4.0f, 8.0f, 8.0f)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("ProjectName", "Project Name"))
				]
				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Top)
				[
					SNew(SBox)
					.WidthOverride(275.0f)
					[
						SNew(SEditableTextBox)
						.IsReadOnly(InArgs._IsReadOnly)
						.Text(InArgs._Name)
						.OnTextChanged(InArgs._OnNameChanged)
						.OnTextCommitted(InArgs._OnNameCommitted)
					]
				]
			]
		];
	}

private:
	EVisibility GetWarningVisibility() const
	{
		return WarningText.Get().IsEmpty() ? EVisibility::Hidden : EVisibility::HitTestInvisible;
	}
private:
	TAttribute<FText> WarningText;
};

TSharedRef<FSlateBrush> SProjectDialog::SetupThumbnailBrush(TSharedRef<FSlateBrush> ThumbnailBrush)
{
	ThumbnailBrush->OutlineSettings.CornerRadii = FVector4(4, 4, 0, 0);
	ThumbnailBrush->OutlineSettings.RoundingType = ESlateBrushRoundingType::FixedRadius;
	ThumbnailBrush->DrawAs = ESlateBrushDrawType::RoundedBox;

	return ThumbnailBrush;
}

void SProjectDialog::Construct(const FArguments& InArgs, EProjectDialogModeMode Mode)
{
	bLastGlobalValidityCheckSuccessful = true;
	bLastNameAndLocationValidityCheckSuccessful = true;

	PopulateTemplateCategories();

	ProjectBrowser = SNew(SProjectBrowser);
	if (UEditorSettings* EditorSettings = GetMutableDefault<UEditorSettings>())
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		bLoadMostRecentlyLoadedProject = EditorSettings->bLoadTheMostRecentlyLoadedProjectAtStartup;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		EditorSettings->OnMostRecentProjectSettingChanged().AddSP(this, &SProjectDialog::OnLoadMostRecentProjectEditorSettingChanged);
	}

	ChildSlot
	[
		SNew(SBorder)
		.BorderImage(FAppStyle::Get().GetBrush("Brushes.Panel"))
		.Padding(FMargin(16.0f, 8.0f))
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.Padding(0.0f, 0.0f, 0.0f, 12.0f)
			[
				MakeHybridView(Mode)
			]
			+SVerticalBox::Slot()
			.AutoHeight()
			.Padding(-16.0f, 0.0f)
			[
				SNew(SSeparator)
				.Orientation(EOrientation::Orient_Horizontal)
				.Thickness(2.0f)
			]
			+SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SBox)
				.HeightOverride(145.0f)
				[
					SAssignNew(PathAreaSwitcher, SWidgetSwitcher)
					+SWidgetSwitcher::Slot()
					[
						MakeNewProjectPathArea()
					]
					+SWidgetSwitcher::Slot()
					[
						MakeOpenProjectPathArea()
					]
				]
			]
		]
	];

	RegisterActiveTimer(1.0f, FWidgetActiveTimerDelegate::CreateLambda(
		[this](double, float)
		{
			UpdateProjectFileValidity();
			return EActiveTimerReturnType::Continue;
		}));

	if (Mode == EProjectDialogModeMode::OpenProject)
	{
		PathAreaSwitcher->SetActiveWidgetIndex(1);
	}
	else if (Mode == EProjectDialogModeMode::Hybrid && ProjectBrowser->HasProjects())
	{
		// Select recent projects
		OnRecentProjectsClicked();
	}
	else if(!ProjectBrowser->HasProjects() || Mode == EProjectDialogModeMode::NewProject)
	{
		// Select the first template category
		MajorCategoryList->SetSelection(TemplateCategories[0]);
	}

}

SProjectDialog::~SProjectDialog()
{
	// Unbind the callback on the setting changed
	if (UEditorSettings* EditorSettings = GetMutableDefault<UEditorSettings>())
	{
		EditorSettings->OnMostRecentProjectSettingChanged().RemoveAll(this);
	}

	// remove any UTemplateProjectDefs we were keeping alive
	for (const TPair<FName, TArray<TSharedPtr<FTemplateItem>>>& Pair : Templates)
	{
		for (const TSharedPtr<FTemplateItem>& Template : Pair.Value)
		{
			if (Template->CodeTemplateDefs != nullptr)
			{
				Template->CodeTemplateDefs->RemoveFromRoot();
			}

			if (Template->BlueprintTemplateDefs != nullptr)
			{
				Template->BlueprintTemplateDefs->RemoveFromRoot();
			}
		}
	}
}

void SProjectDialog::PopulateTemplateCategories()
{
	TemplateCategories.Empty();
	CurrentCategory.Reset();

	TemplateCategories = GetAllTemplateCategories();
}

TSharedRef<SWidget> SProjectDialog::MakeNewProjectDialogButtons()
{
	return
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.Padding(8.0f, 0.0f, 8.0f, 0.0f)
		.VAlign(VAlign_Center)
		.AutoWidth()
		[
			SNew(SGetSuggestedIDEWidget)
			.VisibilityOverride(this, &SProjectDialog::GetSuggestedIDEButtonVisibility)
		]
		+ SHorizontalBox::Slot()
		.Padding(8.0f, 0.0f, 8.0f, 0.0f)
		.VAlign(VAlign_Center)
		.AutoWidth()
		[
			SNew(SGetDisableIDEWidget)
			.Visibility(this, &SProjectDialog::GetDisableIDEButtonVisibility)
		]
		+SHorizontalBox::Slot()
		.Padding(8.0f, 0.0f, 8.0f, 0.0f)
		.VAlign(VAlign_Center)
		.AutoWidth()
		[
			SNew(SPrimaryButton)
			.Visibility(this, &SProjectDialog::GetCreateButtonVisibility)
			.Text(LOCTEXT("CreateNewProject", "Create"))
			.IsEnabled(this, &SProjectDialog::CanCreateProject)
			.OnClicked_Lambda([this](){CreateAndOpenProject(); return FReply::Handled(); })
		]
		+ SHorizontalBox::Slot()
		.Padding(8.0f, 0.0f, 0.0f, 0.0f)
		.VAlign(VAlign_Center)
		.AutoWidth()
		[
			SNew(SButton)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.Text(LOCTEXT("CancelNewProjectCreation", "Cancel"))
			.OnClicked(this, &SProjectDialog::OnCancel)
		];
}

TSharedRef<SWidget> SProjectDialog::MakeOpenProjectDialogButtons()
{
	TSharedRef<SProjectBrowser> ProjectBrowserRef = ProjectBrowser.ToSharedRef();

	return
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.Padding(8.0f, 0.0f)
		.VAlign(VAlign_Center)
		.AutoWidth()
		[
			SNew(SButton)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.Text(LOCTEXT("BrowseForProjects", "Browse..."))
			.ToolTipText(LOCTEXT("BrowseForProjects_Tooltip", "Browse to and open a project on your computer."))
			.OnClicked(ProjectBrowserRef, &SProjectBrowser::OnBrowseToProject)
		]
		+SHorizontalBox::Slot()
		.Padding(8.0f, 0.0f)
		.VAlign(VAlign_Center)
		.AutoWidth()
		[
			SNew(SPrimaryButton)
			.Visibility(this, &SProjectDialog::GetCreateButtonVisibility)
			.Text(LOCTEXT("OpenProject", "Open"))
			.ToolTipText(LOCTEXT("OpenProject_Tooltip", "Open the selected project."))
			.IsEnabled(ProjectBrowserRef, &SProjectBrowser::HasSelectedProjectFile)
			.OnClicked(ProjectBrowserRef, &SProjectBrowser::OnOpenProject)
		]
		+ SHorizontalBox::Slot()
		.Padding(8.0f, 0.0f, 0.0f, 0.0f)
		.VAlign(VAlign_Center)
		.AutoWidth()
		[
			SNew(SButton)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.Text(LOCTEXT("CancelNewProjectCreation", "Cancel"))
			.OnClicked(this, &SProjectDialog::OnCancel)
		];
}

TSharedRef<SWidget> SProjectDialog::MakeTemplateProjectView()
{
	return
		SNew(SVerticalBox)
		// Templates list
		+ SVerticalBox::Slot()
		.FillHeight(1.0f)
		.Padding(0.0f)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.Padding(0.0f, 0.0f, 0.0f, -12.0f)
			[
				SNew(SScrollBorder, TemplateListView.ToSharedRef())
				[
					TemplateListView.ToSharedRef()
				]
			]						
			+ SHorizontalBox::Slot()
			.Padding(0, -8.0f, 0.0f, -12.0f)
			.AutoWidth()
			[
				SNew(SSeparator)
				.Orientation(EOrientation::Orient_Vertical)
				.Thickness(2.0f)
			]
			// Selected template details
			+ SHorizontalBox::Slot()
			.Padding(8.0f, 0.0f, 0.0f, 0.0f)
			.AutoWidth()
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.FillHeight(1.f)
				.VAlign(VAlign_Top)
				[
					// Preview image
					SNew(SVerticalBox)
					+ SVerticalBox::Slot()
					.AutoHeight()
					.Padding(FMargin(0.0f, 0.0f, 0.0f, 15.f))
					[
						SNew(SImage)
						.DesiredSizeOverride(FVector2D(400, 200))
						.Image(this, &SProjectDialog::GetSelectedTemplatePreviewImage)
					]
					// Template Name
					+ SVerticalBox::Slot()
					.Padding(FMargin(0.0f, 0.0f, 0.0f, 10.0f))
					.AutoHeight()
					[
						SNew(STextBlock)
						.AutoWrapText(true)
						.TextStyle(FAppStyle::Get(), "DialogButtonText")
						.Font(FAppStyle::Get().GetFontStyle("HeadingExtraSmall"))
						.ColorAndOpacity(FAppStyle::Get().GetSlateColor("Colors.White"))
						.Text_Lambda([this]()
						{
							if (const TSharedPtr<FTemplateItem> SelectedItem = GetSelectedTemplateItem())
							{
								return SelectedItem->GetDisplayName();
							}
							return FText::GetEmpty();
						})
					]
					// Template Description
					+ SVerticalBox::Slot()
					.FillHeight(1.f)
					[
						SNew(SBox)
						.WidthOverride(358.0f)
						[
							SNew(SScrollBox)
							+ SScrollBox::Slot()
							[ 
								SNew(SVerticalBox)
								+ SVerticalBox::Slot()
								.Padding(FMargin(0.0f, 0.0f, 0.0f, 10.0f))
								.AutoHeight()
								[
									SNew(STextBlock)
									.WrapTextAt(350.0f)
									.Text_Lambda([this]()
									{
										if (const TSharedPtr<FTemplateItem> SelectedItem = GetSelectedTemplateItem())
										{
											return SelectedItem->GetDescription();
										}
										return FText::GetEmpty();
									})
								]
								// Asset types
								+ SVerticalBox::Slot()
								.AutoHeight()
								.Padding(FMargin(0.0f, 5.0f, 0.0f, 5.0f))
								[
									SNew(SVerticalBox)
									.Visibility(this, &SProjectDialog::GetSelectedTemplateAssetVisibility)
									+ SVerticalBox::Slot()
									[
										SNew(STextBlock)
										.TextStyle(FAppStyle::Get(), "DialogButtonText")
										.Font(FAppStyle::Get().GetFontStyle("NormalFontBold"))
										.ColorAndOpacity(FAppStyle::Get().GetSlateColor("Colors.White"))
										.Text(LOCTEXT("ProjectTemplateAssetTypes", "Asset Type References"))
									]
									+ SVerticalBox::Slot()
									.AutoHeight()
									[
										SNew(STextBlock)
										.AutoWrapText(true)
										.Text(this, &SProjectDialog::GetSelectedTemplateAssetTypes)
									]
								]
								// Class types
								+ SVerticalBox::Slot()
								.AutoHeight()
								.Padding(FMargin(0.0f, 5.0f, 0.0f, 5.0f))
								[
									SNew(SVerticalBox)
									.Visibility(this, &SProjectDialog::GetSelectedTemplateClassVisibility)
									+ SVerticalBox::Slot()
									[
										SNew(STextBlock)
										.TextStyle(FAppStyle::Get(), "DialogButtonText")
										.Font(FAppStyle::Get().GetFontStyle("NormalFontBold"))
										.ColorAndOpacity(FAppStyle::Get().GetSlateColor("Colors.White"))
										.Text(LOCTEXT("ProjectTemplateClassTypes", "Class Type References"))
									]
									+ SVerticalBox::Slot()
									.AutoHeight()
									[
										SNew(STextBlock)
										.AutoWrapText(true)
										.Text(this, &SProjectDialog::GetSelectedTemplateClassTypes)
									]
								]
								// Plugin Information
								+ SVerticalBox::Slot()
								.AutoHeight()
								.Padding(FMargin(0.0f, 5.0f, 0.0f, 0.0f))
								.Expose(PluginInformationSlot)
							]
						]
					]
				]
				// Project Options
				+ SVerticalBox::Slot()
				.Padding(FMargin(0.0f, 0.0f, 0.0f, 15.0f))
				.AutoHeight()
				.VAlign(VAlign_Bottom)
				.Expose(ProjectOptionsSlot)
			]
		];
}

TSharedRef<SWidget> SProjectDialog::MakeHybridView(EProjectDialogModeMode Mode)
{
	SelectedHardwareClassTarget = EHardwareClass::Desktop;
	SelectedGraphicsPreset = EGraphicsPreset::Maximum;

	// Find all template projects
	Templates = FindTemplateProjects();
	SetDefaultProjectLocation();

	TemplateListView = SNew(STileView<TSharedPtr<FTemplateItem>>)
		.ListItemsSource(&FilteredTemplateList)
		.SelectionMode(ESelectionMode::Single)
		.ClearSelectionOnClick(false)
		.ItemAlignment(EListItemAlignment::LeftAligned)
		.OnGenerateTile_Static(&STemplateTile::BuildTile)
		.ItemHeight(NewProjectDialogDefs::TemplateTileHeight+9)
		.ItemWidth(NewProjectDialogDefs::TemplateTileWidth+9)
		.OnSelectionChanged(this, &SProjectDialog::HandleTemplateListViewSelectionChanged);

	const UE::Slate::FDeprecateVector2DResult MajorCategorySize = FAppStyle::Get().GetVector("ProjectBrowser.MajorCategory.Size");

	TSharedRef<SWidget> HybridView = 
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		[ 
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.Padding(0.0f, 0.0f, 0.0f, 16.0f)
			.AutoHeight()
			[
				SNew(SBox)
				.Visibility(Mode == EProjectDialogModeMode::Hybrid ? EVisibility::Visible : EVisibility::Collapsed)
				[	
					MakeRecentProjectsTile()
				]
			]
			+ SVerticalBox::Slot()
			.Padding(0.0f, -4.0f, 0.0f, 0.0f)
			[
				SNew(SBorder)
				.Visibility(Mode == EProjectDialogModeMode::OpenProject ? EVisibility::Collapsed : EVisibility::Visible)
				.BorderImage(FAppStyle::Get().GetBrush("ProjectBrowser.MajorCategory.ViewBorder"))
				[
					SAssignNew(MajorCategoryList, STileView<TSharedPtr<FTemplateCategory>>)
					.ListItemsSource(&TemplateCategories)
					.SelectionMode(ESelectionMode::Single)
					.ClearSelectionOnClick(false)
					.OnGenerateTile(this, &SProjectDialog::ConstructMajorCategoryTableRow)
					.ItemHeight(MajorCategorySize.Y)
					.ItemWidth(MajorCategorySize.X)
					.OnSelectionChanged(this, &SProjectDialog::OnMajorTemplateCategorySelectionChanged)
				]
			]
		]
		+SHorizontalBox::Slot()
		.Padding(11.0f, 0.0f, 0.0f, 0.0f)
		[
			SAssignNew(TemplateAndRecentProjectsSwitcher, SWidgetSwitcher)
			+ SWidgetSwitcher::Slot()
			[
				MakeTemplateProjectView()
			]
			+ SWidgetSwitcher::Slot()
			[
				ProjectBrowser.ToSharedRef()
			]
		];
	
	SetCurrentMajorCategory(ActiveCategory);

	if (Mode == EProjectDialogModeMode::OpenProject)
	{
		TemplateAndRecentProjectsSwitcher->SetActiveWidgetIndex(1);
	}

	UpdateProjectFileValidity();

	return HybridView;
}

TSharedRef<SWidget> SProjectDialog::MakeProjectOptionsWidget()
{
	IHardwareTargetingModule& HardwareTargeting = IHardwareTargetingModule::Get();

	TSharedPtr<SVerticalBox> ProjectOptionsBox;

	TSharedRef<SWidget> ProjectOptionsWidget =
		SNew(SVerticalBox)
		.Visibility(this, &SProjectDialog::GetProjectSettingsVisibility)
		+ SVerticalBox::Slot()
		.AutoHeight()
		[	
			SNew(SBorder)
			.Padding(FMargin(10.0f, 7.0f))
			.BorderImage(FAppStyle::Get().GetBrush("Brushes.Header"))
			[
				SNew(STextBlock)	
				.TextStyle(FAppStyle::Get(), "DialogButtonText")
				.Text(LOCTEXT("ProjectDefaults", "Project Defaults"))
			]
		]
		+ SVerticalBox::Slot()
		.Padding(0.0f, 10.f, 0.0f, 0.0f)
		[
			SNew(SScrollBox)
			+ SScrollBox::Slot()
			[
				SAssignNew(ProjectOptionsBox, SVerticalBox)
			]
		];

	const TArray<ETemplateSetting>& HiddenSettings = GetSelectedTemplateProperty(&FTemplateItem::HiddenSettings);

	bool bIsBlueprintAvailable = !GetSelectedTemplateProperty(&FTemplateItem::BlueprintProjectFile).IsEmpty();
	bool bIsCodeAvailable = !GetSelectedTemplateProperty(&FTemplateItem::CodeProjectFile).IsEmpty();

	if (!HiddenSettings.Contains(ETemplateSetting::Languages))
	{
		// if neither is available, then this is a blank template, so both are available
		if (!bIsBlueprintAvailable && !bIsCodeAvailable)
		{
			bIsBlueprintAvailable = true;
			bIsCodeAvailable = true;
		}

		bShouldGenerateCode = !bIsBlueprintAvailable;

		TSharedRef<SSegmentedControl<int32>> ScriptTypeChooser =
			SNew(SSegmentedControl<int32>)
			.UniformPadding(FMargin(25.0f,4.0f))
			
			.ToolTipText(LOCTEXT("ProjectDialog_BlueprintOrCppDescription", "Choose whether to create a Blueprint or C++ project.\nNote: You can also add blueprints to a C++ project and C++ to a Blueprint project later."))
			.Value(this, &SProjectDialog::OnGetBlueprintOrCppIndex)
			.OnValueChanged(this, &SProjectDialog::OnSetBlueprintOrCppIndex);

			if (bIsBlueprintAvailable)
			{
				ScriptTypeChooser->AddSlot(0)
				.HAlign(HAlign_Center)
				.Text(LOCTEXT("ProjectDialog_Blueprint", "BLUEPRINT"));
			}

			if (bIsCodeAvailable)
			{
				ScriptTypeChooser->AddSlot(1)
				.HAlign(HAlign_Center)
				.Text(LOCTEXT("ProjectDialog_Code", "C++"));
			}

			ProjectOptionsBox->AddSlot()
				.AutoHeight()
				.HAlign(HAlign_Center)
				[
					ScriptTypeChooser
				];
	}
	else
	{
		bShouldGenerateCode = bIsCodeAvailable;
	}

	if (!HiddenSettings.Contains(ETemplateSetting::HardwareTarget))
	{
		TSharedRef<SWidget> HardwareClassTarget = HardwareTargeting.MakeHardwareClassTargetCombo(
			FOnHardwareClassChanged::CreateSP(this, &SProjectDialog::SetHardwareClassTarget),
			TAttribute<EHardwareClass>(this, &SProjectDialog::GetHardwareClassTarget));
		
		ProjectOptionsBox->AddSlot()
		.Padding(0.0f, 16.0f, 0.0f, 8.0f)
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			.ToolTipText(LOCTEXT("ProjectDialog_HardwareClassTargetDescription", "Choose the closest equivalent target platform. You can change this later in the Target Hardware section of Project Settings."))
			+ SHorizontalBox::Slot()
			.Padding(0.0f, 0.0f, 8.0f, 0.0f)
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Right)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("TargetPlatform", "Target Platform"))
			]
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Fill)
			[
				SNew(SBox)
				[
					HardwareClassTarget
				]
			]
		];
	}	


	if (!HiddenSettings.Contains(ETemplateSetting::GraphicsPreset))
	{
		TSharedRef<SWidget> GraphicsPreset = HardwareTargeting.MakeGraphicsPresetTargetCombo(
			FOnGraphicsPresetChanged::CreateSP(this, &SProjectDialog::SetGraphicsPreset),
			TAttribute<EGraphicsPreset>(this, &SProjectDialog::GetGraphicsPreset));

		ProjectOptionsBox->AddSlot()
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			.ToolTipText(LOCTEXT("ProjectDialog_GraphicsPresetDescription", "Choose the performance characteristics of your project. You can change this later in the Target Hardware section of Project Settings."))
			+ SHorizontalBox::Slot()
			.Padding(0.0f, 0.0f, 8.0f, 0.0f)
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Right)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("QualityPreset", "Quality Preset"))
			]
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Fill)
			[
				SNew(SBox)
				[
					GraphicsPreset
				]
			]
		];
	}
	
	ResetVariants(); // Needs to be called after we have set bShouldGenerateCode
	const UTemplateProjectDefs* ProjectDefs = GetSelectedTemplateDefs();
	if (!HiddenSettings.Contains(ETemplateSetting::Variants) && ProjectDefs && !ProjectDefs->Variants.IsEmpty())
	{
		ProjectOptionsBox->AddSlot()
			.Padding(0.0f, 8.0f, 0.0f, 0.0f)
			.AutoHeight()
			[
				SNew(SHorizontalBox)
				.ToolTipText(LOCTEXT("Variants_ToolTip", "Which variant would you like to use?"))
				.Visibility(this, &SProjectDialog::GetVariantsVisibility)
				+ SHorizontalBox::Slot()
				.Padding(0.0f, 0.0f, 8.0f, 0.0f)
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Right)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("Variant", "Variant"))
				]
				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Fill)
				[
					SNew(SComboButton)
					.ContentPadding(FMargin(4.0f, 0.0f))
					.OnGetMenuContent(this, &SProjectDialog::GetVariantsDropdownContent)
					.ButtonContent()
					[
						SNew(STextBlock)
						.Text(this, &SProjectDialog::GetVariantsButtonText)
						.ToolTipText(this, &SProjectDialog::GetVariantsButtonTooltip)
					]
				]
			];
	}

#if 0 // @todo: XR settings cannot be shown at the moment as the setting causes issues with binary builds.
	if (!HiddenSettings.Contains(ETemplateSetting::XR))
	{
		TArray<SDecoratedEnumCombo<int32>::FComboOption> VirtualRealityOptions;
		VirtualRealityOptions.Add(SDecoratedEnumCombo<int32>::FComboOption(
			0, FSlateIcon(FAppStyle::GetAppStyleSetName(), "GameProjectDialog.XRDisabled"),
			LOCTEXT("XRDisabled", "XR Disabled")));

		VirtualRealityOptions.Add(SDecoratedEnumCombo<int32>::FComboOption(
			1, 
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "GameProjectDialog.XREnabled"),
			LOCTEXT("XREnabled", "XR Enabled")));

		TSharedRef<SDecoratedEnumCombo<int32>> Enum = SNew(SDecoratedEnumCombo<int32>, MoveTemp(VirtualRealityOptions))
			.SelectedEnum(this, &SProjectDialog::OnGetXREnabled)
			.OnEnumChanged(this, &SProjectDialog::OnSetXREnabled)
			.Orientation(Orient_Vertical);

		TSharedRef<SRichTextBlock> Description = SNew(SRichTextBlock)
			.Text(LOCTEXT("ProjectDialog_XREnabledDescription", "Choose if XR should be enabled in the new project."))
			.AutoWrapText(true)
			.DecoratorStyleSet(&FAppStyle::Get());
	}
#endif

	if (ProjectOptionsBox->NumSlots() == 0)
	{
		return SNullWidget::NullWidget;
	}

	return ProjectOptionsWidget;
}

TSharedRef<SWidget> SProjectDialog::MakePluginInformationWidget()
{
	const TSharedPtr<IPlugin> Plugin = GetSelectedTemplateProperty(&FTemplateItem::PluginWeak).Pin();
	if (!Plugin)
	{
		return SNullWidget::NullWidget;
	}

	const FPluginDescriptor& PluginDescriptor = Plugin->GetDescriptor();

	const ISlateStyle* PluginStyle = FSlateStyleRegistry::FindSlateStyle("PluginStyle");

	const float PaddingAmount = PluginStyle->GetFloat("PluginTile.Padding");
	const float ThumbnailImageSize = PluginStyle->GetFloat("PluginTile.ThumbnailImageSize");

	// Plugin thumbnail image
	FString Icon128FilePath = Plugin->GetBaseDir() / TEXT("Resources/Icon128.png");

	const FName BrushName(*Icon128FilePath);
	const FIntPoint Size = FSlateApplication::Get().GetRenderer()->GenerateDynamicImageResource(BrushName);

	if ((Size.X > 0) && (Size.Y > 0))
	{
		PluginIconDynamicImageBrush = MakeShareable(new FSlateDynamicImageBrush(BrushName, FVector2D(Size.X, Size.Y)));
	}

	const FText PluginName = FText::FromString(Plugin->GetFriendlyName());
	const FText PluginAuthor = FText::FromString(PluginDescriptor.CreatedBy);


	TSharedRef<SWidget> PluginInformationWidget =
		SNew(SVerticalBox)
		.Visibility(this, &SProjectDialog::GetProjectSettingsVisibility)
		// Plugin information header
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(STextBlock)
			.TextStyle(FAppStyle::Get(), "DialogButtonText")
			.Font(FAppStyle::Get().GetFontStyle("NormalFontBold"))
			.ColorAndOpacity(FAppStyle::Get().GetSlateColor("Colors.White"))
			.Text(LOCTEXT("PluginInformation", "Plugin Information"))
		]
		// Plugin tile 
		+ SVerticalBox::Slot()
		.Padding(0.0f, 10.f, 0.0f, 0.0f)
		[
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.Padding(0)
			.AutoHeight()
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::Get().GetBrush("Brushes.Panel"))
				.Padding(0)
				[
					SNew(SHorizontalBox)
					// Thumbnail image
					+ SHorizontalBox::Slot()
					.Padding(PaddingAmount, PaddingAmount, PaddingAmount + 10.f, PaddingAmount)
					.VAlign(VAlign_Center)
					.AutoWidth()
					[
						SNew(SBox)
						.WidthOverride(ThumbnailImageSize)
						.HeightOverride(ThumbnailImageSize)
						[
							SNew(SBorder)
							.BorderImage(PluginStyle->GetBrush("PluginTile.ThumbnailBorderImage"))
							[
								SNew(SImage)
								.Image(PluginIconDynamicImageBrush.IsValid() ? PluginIconDynamicImageBrush.Get() : PluginStyle->GetBrush("PluginTile.ThumbnailImageFallBack"))
							]
						]
					]
					+ SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					[

						SNew(SVerticalBox)
						+ SVerticalBox::Slot()
						.VAlign(VAlign_Top)
						[
							SNew(SHorizontalBox)
							// Friendly Name
							+SHorizontalBox::Slot()
							.VAlign(VAlign_Center)
							.HAlign(HAlign_Left)
							[
								SNew(STextBlock)
								.Text(PluginName)
								.OverflowPolicy(ETextOverflowPolicy::Ellipsis)
								.ToolTipText(PluginName)
								.TextStyle(FAppStyle::Get(), "DialogButtonText")
								.ColorAndOpacity(FAppStyle::Get().GetSlateColor("Colors.White"))
							]
							// Created By
							+ SHorizontalBox::Slot()
							.VAlign(VAlign_Center)
							.HAlign(HAlign_Right)
							[
								SNew(STextBlock)
								.Text(PluginAuthor)
								.TextStyle(FAppStyle::Get(), "DialogButtonText")
							]
						]
						// Version
						+ SVerticalBox::Slot()
						.AutoHeight()
						.VAlign(VAlign_Bottom)
						.HAlign(HAlign_Left)
						.Padding(0.0f, PaddingAmount, 0.0f, 0.0f)
						[
							SNew(SHorizontalBox)
							// version number
							+ SHorizontalBox::Slot()
							.AutoWidth()
							.VAlign(VAlign_Center)
							.Padding(0.0f, 0.0f, 0.0f, 1.0f) // Lower padding to align font with version number base
							[
								SNew(STextBlock)
								.Text(FText::Format(LOCTEXT("PluginVersionLabel", "Version {0}"), FText::FromString(PluginDescriptor.VersionName)))
								.TextStyle(FAppStyle::Get(), "DialogButtonText")
							]
						]
					]
				]
			]
		];

	return PluginInformationWidget;
}

TSharedRef<SWidget> SProjectDialog::MakeRecentProjectsTile()
{
	RecentProjectsCategory = MakeShared<FTemplateCategory>();

	RecentProjectsCategory->DisplayNameOverride = LOCTEXT("RecentProjects", "Recent Projects");
	RecentProjectsCategory->DescriptionOverride = FText::GetEmpty();
	RecentProjectsCategory->Key = "RecentProjects";
	RecentProjectsCategory->IsEnterprise = false;

	static const FName BrushName =  *(FAppStyle::Get().GetContentRootDir() / TEXT("/Starship/Projects/") / TEXT("RecentProjects_2x.png"));

	RecentProjectsBrush = SetupThumbnailBrush(MakeShared<FSlateDynamicImageBrush>(BrushName, FVector2D(300, 100)));
	RecentProjectsCategory->Icon = RecentProjectsBrush;

	return
		SNew(SButton)
		.ButtonStyle(FAppStyle::Get(), "InvisibleButton")
		.OnClicked(this, &SProjectDialog::OnRecentProjectsClicked)
		.ForegroundColor(FLinearColor::White)
		.ContentPadding(FMargin(4.0f, 0.0f))
		[
			SNew(SMajorCategoryTile, RecentProjectsCategory)
			.IsSelected_Lambda([this]() { return RecentProjectsCategory == CurrentCategory; })
		];
}

TSharedRef<SWidget> SProjectDialog::MakeNewProjectPathArea()
{
	return 
		SNew(SVerticalBox)
		+SVerticalBox::Slot()
		.Padding(25.0f, 36.0f, 0.0f, 0)
		[
			SNew(SFilepath)
			.OnBrowseForFolder(this, &SProjectDialog::HandlePathBrowseButtonClicked)
			.FolderPath(this, &SProjectDialog::GetCurrentProjectFilePath)
			.WarningText(this, &SProjectDialog::GetNameAndLocationValidityErrorText)
			.Name(this, &SProjectDialog::GetCurrentProjectFileName)
			.OnFolderChanged(this, &SProjectDialog::OnCurrentProjectFilePathChanged)
			.OnNameChanged(this, &SProjectDialog::OnCurrentProjectFileNameChanged)
		]
		+SVerticalBox::Slot()
		.Padding(0.0f, 0.0f, 0.0f, 8.0f)
		.VAlign(VAlign_Bottom)
		.HAlign(HAlign_Right)
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				SNew(SWarningOrErrorBox)
				.Padding(FMargin(8.0f, 4.0f, 4.0f, 4.0f))
				.IconSize(FVector2D(16, 16))
				.MessageStyle(EMessageStyle::Warning)
				.Message(this, &SProjectDialog::GetGlobalWarningLabelText)
				.Visibility(this, &SProjectDialog::GetGlobalWarningVisibility)
			]
			+SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.AutoWidth()
			.Padding(FMargin(8.0f, 0.0f, 0.0f, 0.0f))
			[
				SNew(SWarningOrErrorBox)
				.Padding(FMargin(8.0f, 4.0f, 4.0f, 4.0f))
				.IconSize(FVector2D(16,16))
				.MessageStyle(EMessageStyle::Error)
				.Message(this, &SProjectDialog::GetGlobalErrorLabelText)
				.Visibility(this, &SProjectDialog::GetGlobalErrorVisibility)
			]
			+SHorizontalBox::Slot()
			.VAlign(VAlign_Bottom)
			[
				MakeNewProjectDialogButtons()
			]
		];
}

TSharedRef<SWidget> SProjectDialog::MakeOpenProjectPathArea()
{
	return
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.Padding(25.0f, 36.0f, 0.0f, 0)
		[
			SNew(SFilepath)
			.IsReadOnly(true)
			.FolderPath(this, &SProjectDialog::GetCurrentProjectFilePath)
			.Name(this, &SProjectDialog::GetCurrentProjectFileName)
		]
		+SVerticalBox::Slot()
		.Padding(25.0f, 0.0f, 0.0f, 8.0f)
		.VAlign(VAlign_Bottom)	
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			[
				SNew(SCheckBox)
				.IsChecked(this, &SProjectDialog::IsLoadMostRecentProjectChecked)
				.OnCheckStateChanged(ProjectBrowser.ToSharedRef(), &SProjectBrowser::OnAutoloadLastProjectChanged)
				.Padding(FMargin(4.0f, 0.0f))
				[
					SNew(STextBlock)
					.Text(LOCTEXT("AutoloadOnStartupCheckbox", "Always load last project on startup"))
					.ColorAndOpacity(FSlateColor::UseForeground())
				]
			]
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Bottom)
			.HAlign(HAlign_Right)
			[
				MakeOpenProjectDialogButtons()
			]
		];
}

bool SProjectDialog::CanCreateProject() const
{
	return bLastGlobalValidityCheckSuccessful && bLastNameAndLocationValidityCheckSuccessful;
}

FReply SProjectDialog::OnCancel() const
{
	TSharedPtr<SWindow> Window = FSlateApplication::Get().FindWidgetWindow(AsShared());
	Window->RequestDestroyWindow();

	return FReply::Handled();
}

void SProjectDialog::OnSetBlueprintOrCppIndex(int32 Index)
{
	bShouldGenerateCode = Index == 1;
	UpdateProjectFileValidity();
	FProjectInformation ProjectInfo = CreateProjectInfo();

	ResetVariants();
}

void SProjectDialog::SetHardwareClassTarget(EHardwareClass InHardwareClass)
{
	SelectedHardwareClassTarget = InHardwareClass;
}

void SProjectDialog::SetGraphicsPreset(EGraphicsPreset InGraphicsPreset)
{
	SelectedGraphicsPreset = InGraphicsPreset;
}

EVisibility SProjectDialog::GetVariantsVisibility() const
{
	const UTemplateProjectDefs* ProjectDefs = GetSelectedTemplateDefs();
	return ProjectDefs && !ProjectDefs->Variants.IsEmpty() ? EVisibility::Visible : EVisibility::Collapsed;
}

TSharedRef<SWidget> SProjectDialog::GetVariantsDropdownContent()
{
	FMenuBuilder MenuBuilder(true, nullptr, nullptr, true);

	for (const FName& VariantName : TemplateVariantNames)
	{
		const FTemplateVariant* Variant = GetSelectedTemplateDefsVariant(VariantName);
		MenuBuilder.AddMenuEntry(
			Variant ? Variant->GetDisplayNameText() : LOCTEXT("NoVariant", "None"),
			Variant ? Variant->GetLocalizedDescription() : LOCTEXT("NoVariantDescription", "Do not include a variant"),
			FSlateIcon(),
			FUIAction
			(
				FExecuteAction::CreateLambda([this, VariantName]()
				{
					SelectedVariantName = VariantName;
				})
			)
		);
	}

	return MenuBuilder.MakeWidget();
}

FText SProjectDialog::GetVariantsButtonText() const
{
	const FTemplateVariant* Variant = GetSelectedTemplateDefsVariant();
    return Variant ? Variant->GetDisplayNameText() : LOCTEXT("NoVariant", "None");
}

FText SProjectDialog::GetVariantsButtonTooltip() const
{
	const FTemplateVariant* Variant = GetSelectedTemplateDefsVariant();
	return Variant ? Variant->GetLocalizedDescription() : LOCTEXT("NoVariantDescription", "Do not include a variant");
}

void SProjectDialog::ResetVariants()
{
	// Recreate the VariantNames. BP and CPP templates can have different variants
	TemplateVariantNames.Empty();
	TemplateVariantNames.Add(NAME_None); // The user can choose to have no variants
	
	const UTemplateProjectDefs* ProjectDefs = GetSelectedTemplateDefs();
	if (ProjectDefs)
	{
		Algo::Transform(ProjectDefs->Variants, TemplateVariantNames, [](const FTemplateVariant& Variant) { return Variant.Name; });
	}
	// Check if this template has a variant of the same name as the one previously selected, otherwise resets it
	const bool bFoundVariant = ProjectDefs && ProjectDefs->FindVariant(SelectedVariantName);
	if (!bFoundVariant)
	{
		SelectedVariantName = NAME_None;
	}
}

void SProjectDialog::HandleTemplateListViewSelectionChanged(TSharedPtr<FTemplateItem> TemplateItem, ESelectInfo::Type SelectInfo)
{
	(*ProjectOptionsSlot)
	[
		MakeProjectOptionsWidget()
	];

	(*PluginInformationSlot)
	[
		MakePluginInformationWidget()
	];
	

	UpdateProjectFileValidity();

	UpdateTemplateTrustWarning();
}

TSharedPtr<FTemplateItem> SProjectDialog::GetSelectedTemplateItem() const
{
	TArray<TSharedPtr<FTemplateItem>> SelectedItems = TemplateListView->GetSelectedItems();
	if (SelectedItems.Num() > 0)
	{
		return SelectedItems[0];
	}

	return nullptr;
}

UTemplateProjectDefs* SProjectDialog::GetSelectedTemplateDefs() const
{
	if (const TSharedPtr<FTemplateItem> TemplateItem = GetSelectedTemplateItem())
	{
		return bShouldGenerateCode ? TemplateItem->CodeTemplateDefs : TemplateItem->BlueprintTemplateDefs;
	}
	return nullptr;
}

const FTemplateVariant* SProjectDialog::GetSelectedTemplateDefsVariant(FName VariantName) const
{
	const UTemplateProjectDefs* ProjectDefs = GetSelectedTemplateDefs();
	return ProjectDefs ? ProjectDefs->FindVariant(VariantName) : nullptr;
}

namespace
{
	FString MakeSortKey(const FString& TemplateKey)
	{
		FString Output = TemplateKey;

#if PLATFORM_LINUX
		// Paths with a leading "/" would get sorted before the magic value used for blank projects: "_1"
		Output.RemoveFromStart("/");
#endif

		return Output;
	}
}

TMap<FName, TArray<TSharedPtr<FTemplateItem>> > SProjectDialog::FindTemplateProjects()
{
	// Clear the list out first - or we could end up with duplicates
	TMap<FName, TArray<TSharedPtr<FTemplateItem>>> Templates;

	struct FTemplateFolderMetadata
	{
		FString Folder;
		TWeakPtr<IPlugin> PluginWeak;
	};
	// Now discover and all data driven templates
	TArray<FTemplateFolderMetadata> TemplateRootFolders;

	// @todo rocket make template folder locations extensible.
	TemplateRootFolders.Add({ FPaths::RootDir() + TEXT("Templates"), nullptr });

	// Add the Enterprise templates
	TemplateRootFolders.Add({ FPaths::EnterpriseDir() + TEXT("Templates"), nullptr });

	// Allow plugins to define templates
	TArray<TSharedRef<IPlugin>> Plugins = IPluginManager::Get().GetDiscoveredPlugins();
	for (const TSharedRef<IPlugin>& Plugin : Plugins)
	{
		FString PluginDirectory = Plugin->GetBaseDir();
		if (!PluginDirectory.IsEmpty())
		{
			const FString PluginTemplatesDirectory = FPaths::Combine(*PluginDirectory, TEXT("Templates"));

			if (IFileManager::Get().DirectoryExists(*PluginTemplatesDirectory))
			{
				TemplateRootFolders.Add({ PluginTemplatesDirectory, Plugin.ToWeakPtr() });
			}
		}
	}


	// Form a list of all folders that could contain template projects
	TArray<FTemplateFolderMetadata> AllTemplateFolders;
	for (const auto& [RootFolder, Plugin] : TemplateRootFolders)
	{
		const FString SearchString = RootFolder / TEXT("*");
		TArray<FString> TemplateFolders;
		IFileManager::Get().FindFiles(TemplateFolders, *SearchString, /*Files=*/false, /*Directories=*/true);

		for (const FString& Folder : TemplateFolders)
		{
			AllTemplateFolders.Add({ RootFolder / Folder, Plugin });
		}
	}

	TArray<TSharedPtr<FTemplateItem>> FoundTemplates;

	// Add a template item for every discovered project
	for (const auto& [Folder, Plugin] : AllTemplateFolders)
	{
		const FString SearchString = Folder / TEXT("*.") + FProjectDescriptor::GetExtension();
		TArray<FString> FoundProjectFiles;
		IFileManager::Get().FindFiles(FoundProjectFiles, *SearchString, /*Files=*/true, /*Directories=*/false);

		if (FoundProjectFiles.Num() == 0 || !ensure(FoundProjectFiles.Num() == 1))
		{
			continue;
		}

		// Make sure a TemplateDefs.ini file exists
		UTemplateProjectDefs* TemplateDefs = GameProjectUtils::LoadTemplateDefs(Folder);
		if (TemplateDefs == nullptr)
		{
			continue;
		}

		// we don't have an appropriate referencing UObject to keep these alive with, so we need to keep these template defs alive from GC
		TemplateDefs->AddToRoot();

		// Ignore any templates whose definition says we cannot use to create a project
		if (TemplateDefs->bAllowProjectCreation == false)
		{
			continue;
		}

		const FString ProjectFile = Folder / FoundProjectFiles[0];

		// If no template category was specified, use the default category
		TArray<FName> TemplateCategoryNames = TemplateDefs->Categories;
		if (TemplateCategoryNames.Num() == 0)
		{
			TemplateCategoryNames.Add(NewProjectDialogDefs::DefaultCategoryName);
		}

		// Find a duplicate project, eg. "Foo" and "FooBP"
		FString TemplateKey = Folder;
		TemplateKey.RemoveFromEnd("BP");

		TSharedPtr<FTemplateItem>* ExistingTemplate = FoundTemplates.FindByPredicate([&TemplateKey](TSharedPtr<FTemplateItem> Item)
			{
				return Item->Key == TemplateKey;
			});

		TSharedPtr<FTemplateItem> Template;

		// Create a new template if none was found
		if (ExistingTemplate != nullptr)
		{
			Template = *ExistingTemplate;
		}
		else
		{
			Template = MakeShareable(new FTemplateItem());
		}

		if (TemplateDefs->GeneratesCode(Folder))
		{
			Template->CodeProjectFile = ProjectFile;
			Template->CodeTemplateDefs = TemplateDefs;
		}
		else
		{
			Template->BlueprintProjectFile = ProjectFile;
			Template->BlueprintTemplateDefs = TemplateDefs;
		}

		// The rest has already been set by the existing template, so skip it.
		if (ExistingTemplate != nullptr)
		{
			continue;
		}

		// Did not find an existing template. Create a new one to add to the template list.
		Template->Key = TemplateKey;

		// @todo: These are all basically just copies of what's in UTemplateProjectDefs, but ignore differences between code and BP 
		Template->Categories = TemplateCategoryNames;
		Template->LocalizedDisplayNames = TemplateDefs->LocalizedDisplayNames;
		Template->LocalizedDescriptions = TemplateDefs->LocalizedDescriptions;
		Template->ClassTypes = TemplateDefs->ClassTypes;
		Template->AssetTypes = TemplateDefs->AssetTypes;
		Template->HiddenSettings = TemplateDefs->HiddenSettings;
		Template->bIsEnterprise = TemplateDefs->bIsEnterprise;
		Template->bIsBlankTemplate = TemplateDefs->bIsBlank;
		Template->bThumbnailAsIcon = TemplateDefs->bThumbnailAsIcon;
		Template->PluginWeak = Plugin;

		const FString ThumbnailPNGFile = (Folder + TEXT("/Media/") + FoundProjectFiles[0]).Replace(TEXT(".uproject"), TEXT(".png"));
		if (FPlatformFileManager::Get().GetPlatformFile().FileExists(*ThumbnailPNGFile))
		{
			const FName BrushName = FName(*ThumbnailPNGFile);
			Template->Thumbnail = SetupThumbnailBrush(MakeShared<FSlateDynamicImageBrush>(BrushName, FVector2D(128, 128)));
		}

		TSharedPtr<FSlateDynamicImageBrush> PreviewBrush;
		const FString PreviewPNGFile = (Folder + TEXT("/Media/") + FoundProjectFiles[0]).Replace(TEXT(".uproject"), TEXT("_Preview.png"));
		if (FPlatformFileManager::Get().GetPlatformFile().FileExists(*PreviewPNGFile))
		{
			const FName BrushName = FName(*PreviewPNGFile);
			Template->PreviewImage = MakeShareable(new FSlateDynamicImageBrush(BrushName, FVector2D(512, 256)));
		}

		Template->SortKey = TemplateDefs->SortKey;
		if (Template->SortKey.IsEmpty())
		{
			Template->SortKey = MakeSortKey(TemplateKey);
		}

		FoundTemplates.Add(Template);
	}

	for (const TSharedPtr<FTemplateItem>& Template : FoundTemplates)
	{
		for (const FName& Category : Template->Categories)
		{
			Templates.FindOrAdd(Category).Add(Template);
		}
	}

	TArray<TSharedPtr<FTemplateCategory>> AllTemplateCategories = GetAllTemplateCategories();

	// Validate that all our templates have a category defined
	TArray<FName> CategoryKeys;
	Templates.GetKeys(CategoryKeys);
	for (const FName& CategoryKey : CategoryKeys)
	{
		bool bCategoryExists = AllTemplateCategories.ContainsByPredicate([&CategoryKey](const TSharedPtr<FTemplateCategory>& Category)
			{
				return Category->Key == CategoryKey;
			});

		if (!bCategoryExists)
		{
			UE_LOGF(LogGameProjectGeneration, Warning, "Failed to find category definition named '%ls', it is not defined in any TemplateCategories.ini.", *CategoryKey.ToString());
		}
	}

	// Add blank template to empty categories
	{
		TSharedPtr<FTemplateItem> BlankTemplate = MakeShareable(new FTemplateItem());
		BlankTemplate->DisplayNameOverride = LOCTEXT("BlankProjectName", "Blank");
		BlankTemplate->DescriptionOverride = LOCTEXT("BlankProjectDescription", "A clean empty project with no code and default settings.");
		BlankTemplate->Key = TEXT("Blank");
		BlankTemplate->SortKey = TEXT("_1");
		BlankTemplate->Thumbnail = SetupThumbnailBrush(MakeShared<FSlateBrush>(*FAppStyle::GetBrush("GameProjectDialog.BlankProjectThumbnail")));
		BlankTemplate->PreviewImage = MakeShareable(new FSlateBrush(*FAppStyle::GetBrush("GameProjectDialog.BlankProjectPreview")));
		BlankTemplate->BlueprintProjectFile = TEXT("");
		BlankTemplate->CodeProjectFile = TEXT("");
		BlankTemplate->bIsEnterprise = false;
		BlankTemplate->bIsBlankTemplate = true;

		for (const TSharedPtr<FTemplateCategory>& Category : AllTemplateCategories)
		{
			const TArray<TSharedPtr<FTemplateItem>>* CategoryEntry = Templates.Find(Category->Key);
			if (CategoryEntry == nullptr)
			{
				Templates.Add(Category->Key).Add(BlankTemplate);
			}
		}
	}

	return Templates;
}


void SProjectDialog::SetDefaultProjectLocation()
{
	FString DefaultProjectFilePath;

	// First, try and use the first previously used path that still exists
	for (const FString& CreatedProjectPath : GetDefault<UEditorSettings>()->CreatedProjectPaths)
	{
		if (IFileManager::Get().DirectoryExists(*CreatedProjectPath))
		{
			DefaultProjectFilePath = CreatedProjectPath;
			break;
		}
	}

	if (DefaultProjectFilePath.IsEmpty())
	{
		// No previously used path, decide a default path.
		DefaultProjectFilePath = FDesktopPlatformModule::Get()->GetDefaultProjectCreationPath();
		IFileManager::Get().MakeDirectory(*DefaultProjectFilePath, true);
	}

	if (DefaultProjectFilePath.EndsWith(TEXT("/")))
	{
		DefaultProjectFilePath.LeftChopInline(1);
	}

	FPaths::NormalizeFilename(DefaultProjectFilePath);
	FPaths::MakePlatformFilename(DefaultProjectFilePath);
	const FString GenericProjectName = LOCTEXT("DefaultProjectName", "MyProject").ToString();
	FString ProjectName = GenericProjectName;

	// Check to make sure the project file doesn't already exist
	FText FailReason;
	if (!GameProjectUtils::IsValidProjectFileForCreation(DefaultProjectFilePath / ProjectName / ProjectName + TEXT(".") + FProjectDescriptor::GetExtension(), FailReason))
	{
		// If it exists, find an appropriate numerical suffix
		const int MaxSuffix = 1000;
		int32 Suffix;
		for (Suffix = 2; Suffix < MaxSuffix; ++Suffix)
		{
			ProjectName = GenericProjectName + FString::Printf(TEXT("%d"), Suffix);
			if (GameProjectUtils::IsValidProjectFileForCreation(DefaultProjectFilePath / ProjectName / ProjectName + TEXT(".") + FProjectDescriptor::GetExtension(), FailReason))
			{
				// Found a name that is not taken. Break out.
				break;
			}
		}

		if (Suffix >= MaxSuffix)
		{
			UE_LOGF(LogGameProjectGeneration, Warning, "Failed to find a suffix for the default project name");
			ProjectName = TEXT("");
		}
	}

	if (!DefaultProjectFilePath.IsEmpty())
	{
		CurrentProjectFileName = ProjectName;
		CurrentProjectFilePath = DefaultProjectFilePath;
		FPaths::MakePlatformFilename(CurrentProjectFilePath);
		LastBrowsePath = CurrentProjectFilePath;
	}
}

void SProjectDialog::SetCurrentMajorCategory(FName Category)
{
	FilteredTemplateList = Templates.FindRef(Category);

	// Sort the template folders
	FilteredTemplateList.Sort([](const TSharedPtr<FTemplateItem>& A, const TSharedPtr<FTemplateItem>& B) {
		return A->SortKey < B->SortKey;
		});

	if (FilteredTemplateList.Num() > 0)
	{
		TemplateListView->SetSelection(FilteredTemplateList[0]);
	}
	TemplateListView->RequestListRefresh();

	ActiveCategory = Category;
}

FReply SProjectDialog::OnRecentProjectsClicked()
{
	CurrentCategory = RecentProjectsCategory;

	ActiveCategory = NAME_None;

	MajorCategoryList->ClearSelection();

	TemplateAndRecentProjectsSwitcher->SetActiveWidgetIndex(1);
	PathAreaSwitcher->SetActiveWidgetIndex(1);

	return FReply::Handled();
}

void SProjectDialog::OnLoadMostRecentProjectEditorSettingChanged(bool bInAutoLoadOption)
{
	if (bLoadMostRecentlyLoadedProject != bInAutoLoadOption)
	{
		bLoadMostRecentlyLoadedProject = bInAutoLoadOption;
	}
}

ECheckBoxState SProjectDialog::IsLoadMostRecentProjectChecked() const
{
	return bLoadMostRecentlyLoadedProject ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

FProjectInformation SProjectDialog::CreateProjectInfo() const
{
	TSharedPtr<FTemplateItem> SelectedTemplate = GetSelectedTemplateItem();
	if (!SelectedTemplate.IsValid())
	{
		return FProjectInformation();
	}

	FProjectInformation ProjectInfo;
	ProjectInfo.bShouldGenerateCode = bShouldGenerateCode;
	ProjectInfo.TemplateFile = bShouldGenerateCode ? SelectedTemplate->CodeProjectFile : SelectedTemplate->BlueprintProjectFile;
	ProjectInfo.Variant = SelectedVariantName;
	ProjectInfo.TemplateCategory = ActiveCategory;
	ProjectInfo.bIsEnterpriseProject = SelectedTemplate->bIsEnterprise;
	ProjectInfo.bIsBlankTemplate = SelectedTemplate->bIsBlankTemplate;

	const TArray<ETemplateSetting>& HiddenSettings = SelectedTemplate->HiddenSettings;

	if (!HiddenSettings.Contains(ETemplateSetting::All))
	{
		if (!HiddenSettings.Contains(ETemplateSetting::HardwareTarget))
		{
			ProjectInfo.TargetedHardware = SelectedHardwareClassTarget;
		}

		if (!HiddenSettings.Contains(ETemplateSetting::GraphicsPreset))
		{
			ProjectInfo.DefaultGraphicsPerformance = SelectedGraphicsPreset;
		}

		if (!HiddenSettings.Contains(ETemplateSetting::XR))
		{
			ProjectInfo.bEnableXR = bEnableXR;
		}
	}

	return MoveTemp(ProjectInfo);
}

bool SProjectDialog::CreateProject(const FString& ProjectFile)
{
	// Get the selected template
	TSharedPtr<FTemplateItem> SelectedTemplate = GetSelectedTemplateItem();

	if (!ensure(SelectedTemplate.IsValid()))
	{
		// A template must be selected.
		return false;
	}

	FText FailReason, FailLog;

	FProjectInformation ProjectInfo = CreateProjectInfo();
	ProjectInfo.ProjectFilename = ProjectFile;

	if (!GameProjectUtils::CreateProject(ProjectInfo, FailReason, FailLog))
	{
		SOutputLogDialog::Open(LOCTEXT("CreateProject", "Create Project"), FailReason, FailLog, FText::GetEmpty());
		return false;
	}

	// Successfully created the project. Update the last created location string.
	FString CreatedProjectPath = FPaths::GetPath(FPaths::GetPath(ProjectFile));

	// If the original path was the drives root (ie: C:/) the double path call strips the last /
	if (CreatedProjectPath.EndsWith(":"))
	{
		CreatedProjectPath.AppendChar('/');
	}

	UEditorSettings* Settings = GetMutableDefault<UEditorSettings>();
	Settings->CreatedProjectPaths.Remove(CreatedProjectPath);
	Settings->CreatedProjectPaths.Insert(CreatedProjectPath, 0);
	Settings->PostEditChange();

	return true;
}

void SProjectDialog::CreateAndOpenProject()
{
	if (!CanCreateProject())
	{
		return;
	}

	FString ProjectFile = GetProjectFilenameWithPath();
	if (!CreateProject(ProjectFile))
	{
		return;
	}

	if (bShouldGenerateCode)
	{
		// If the engine is installed it is already compiled, so we can try to build and open a new project immediately. Non-installed situations might require building
		// the engine (especially the case when binaries came from P4), so we only open the IDE for that.
		if (FApp::IsEngineInstalled())
		{
			if (GameProjectUtils::BuildCodeProject(ProjectFile))
			{
				OpenCodeIDE(ProjectFile);
				OpenProject(ProjectFile);
			}
			else
			{
				// User will have already been prompted to open the IDE
			}
		}
		else
		{
			OpenCodeIDE(ProjectFile);
		}
	}
	else
	{
		OpenProject(ProjectFile);
	}
}

bool SProjectDialog::OpenProject(const FString& ProjectFile)
{
	FText FailReason;
	if (GameProjectUtils::OpenProject(ProjectFile, FailReason))
	{
		// Successfully opened the project, the editor is closing.
		// Close this window in case something prevents the editor from closing (save dialog, quit confirmation, etc)
		CloseWindowIfAppropriate();
		return true;
	}

	DisplayError(FailReason);
	return false;
}

void SProjectDialog::CloseWindowIfAppropriate(bool ForceClose)
{
	if (ForceClose || FApp::HasProjectName())
	{
		TSharedPtr<SWindow> ContainingWindow = FSlateApplication::Get().FindWidgetWindow(AsShared());

		if (ContainingWindow.IsValid())
		{
			ContainingWindow->RequestDestroyWindow();
		}
	}
}

void SProjectDialog::DisplayError(const FText& ErrorText)
{
	FString ErrorString = ErrorText.ToString();
	UE_LOGF(LogGameProjectGeneration, Log, "%ls", *ErrorString);
	if(ErrorString.Contains("\n"))
	{
		FMessageDialog::Open(EAppMsgType::Ok, ErrorText);
	}
	else
	{
		PersistentGlobalErrorLabelText = ErrorText;
	}
}

void SProjectDialog::DisplayWarning(const FText& WarningText)
{
	FString WarningString = WarningText.ToString();
	UE_LOGF(LogGameProjectGeneration, Log, "%ls", *WarningString);
	PersistentGlobalWarningLabelText = WarningText;
}

bool SProjectDialog::OpenCodeIDE(const FString& ProjectFile)
{
	FText FailReason;
    
#if PLATFORM_MAC
    // Modern Xcode projects are different based on Desktop/Mobile
    FString Extension = FPaths::GetExtension(ProjectFile);
    FString ModernXcodeProjectFile = ProjectFile;
    if (SelectedHardwareClassTarget == EHardwareClass::Desktop)
    {
        ModernXcodeProjectFile.RemoveFromEnd(TEXT(".") + Extension);
        ModernXcodeProjectFile += TEXT(" (Mac).") + Extension;
    }
    else if (SelectedHardwareClassTarget == EHardwareClass::Mobile)
    {
        ModernXcodeProjectFile.RemoveFromEnd(TEXT(".") + Extension);
        ModernXcodeProjectFile += TEXT(" (IOS).") + Extension;
    }
#endif
    
	if (
#if PLATFORM_MAC
        GameProjectUtils::OpenCodeIDE(ModernXcodeProjectFile, FailReason) ||
        // if modern failed, try again with legacy project name
#endif
        GameProjectUtils::OpenCodeIDE(ProjectFile, FailReason))
	{
		// Successfully opened code editing IDE, the editor is closing
		// Close this window in case something prevents the editor from closing (save dialog, quit confirmation, etc)
		CloseWindowIfAppropriate(true);
		return true;
	}

	DisplayError(FailReason);
	return false;
}


TArray<TSharedPtr<FTemplateCategory>> SProjectDialog::GetAllTemplateCategories()
{
	TArray<TSharedPtr<FTemplateCategory>> AllTemplateCategories;
	FGameProjectGenerationModule::Get().GetAllTemplateCategories(AllTemplateCategories);

	if (AllTemplateCategories.Num() == 0)
	{
		static const FName BrushName = *(FAppStyle::Get().GetContentRootDir() / TEXT("/Starship/Projects/") / TEXT("CustomTemplate_2x.png"));

		if (!CustomTemplateBrush)
		{
			CustomTemplateBrush = SetupThumbnailBrush(MakeShared<FSlateDynamicImageBrush>(BrushName, FVector2D(300, 100)));
		}

		TSharedPtr<FTemplateCategory> DefaultCategory = MakeShared<FTemplateCategory>();
		DefaultCategory->Key = NewProjectDialogDefs::BlankCategoryKey;
		DefaultCategory->DisplayNameOverride = LOCTEXT("ProjectDialog_DefaultCategoryName", "Blank Project");
		DefaultCategory->DescriptionOverride = LOCTEXT("ProjectDialog_DefaultCategoryDescription", "Create a new blank Unreal project.");
		DefaultCategory->Icon = CustomTemplateBrush;

		AllTemplateCategories.Add(DefaultCategory);
	}

	return AllTemplateCategories;
}



FText SProjectDialog::GetGlobalErrorLabelText() const
{
	if (!PersistentGlobalErrorLabelText.IsEmpty())
	{
		return PersistentGlobalErrorLabelText;
	}

	if (!bLastGlobalValidityCheckSuccessful)
	{
		return LastGlobalValidityErrorText;
	}

	return FText::GetEmpty();
}

FText SProjectDialog::GetGlobalWarningLabelText() const
{
	return PersistentGlobalWarningLabelText;
}

FText SProjectDialog::GetNameAndLocationValidityErrorText() const
{
	if (GetGlobalErrorLabelText().IsEmpty())
	{
		return bLastNameAndLocationValidityCheckSuccessful == false ? LastNameAndLocationValidityErrorText : FText::GetEmpty();
	}

	return FText::GetEmpty();
}

EVisibility SProjectDialog::GetCreateButtonVisibility() const
{
	return IsCompilerRequired() && !FSourceCodeNavigation::IsCompilerAvailable() ? EVisibility::Collapsed : EVisibility::Visible;
}

EVisibility SProjectDialog::GetSuggestedIDEButtonVisibility() const
{
	return IsCompilerRequired() && !FSourceCodeNavigation::IsCompilerAvailable() ? EVisibility::Visible : EVisibility::Collapsed;
}

// Allow disabling of the current IDE for platforms that dont require an IDE to run the Editor/Engine
EVisibility SProjectDialog::GetDisableIDEButtonVisibility() const
{
	if (GetSuggestedIDEButtonVisibility() == EVisibility::Visible && !IsIDERequired())
	{
		return EVisibility::Visible;
	}

	return EVisibility::Collapsed;
}

const FSlateBrush* SProjectDialog::GetSelectedTemplatePreviewImage() const
{
	TSharedPtr<FSlateBrush> PreviewImage = GetSelectedTemplateProperty(&FTemplateItem::PreviewImage);
	return PreviewImage.IsValid() ? PreviewImage.Get() : nullptr;
}


FText SProjectDialog::GetCurrentProjectFilePath() const
{
	return PathAreaSwitcher && PathAreaSwitcher->GetActiveWidgetIndex() == 1 ? FText::FromString(ProjectBrowser->GetSelectedProjectFile()) : FText::FromString(CurrentProjectFilePath);
}

void SProjectDialog::OnCurrentProjectFilePathChanged(const FText& InValue)
{
	CurrentProjectFilePath = InValue.ToString();
	FPaths::MakePlatformFilename(CurrentProjectFilePath);
	UpdateProjectFileValidity();
}

void SProjectDialog::OnCurrentProjectFileNameChanged(const FText& InValue)
{
	CurrentProjectFileName = InValue.ToString();
	UpdateProjectFileValidity();
}

FText SProjectDialog::GetCurrentProjectFileName() const
{
	return PathAreaSwitcher && PathAreaSwitcher->GetActiveWidgetIndex() == 1 ? ProjectBrowser->GetSelectedProjectName() : FText::FromString(CurrentProjectFileName);
}

FReply SProjectDialog::HandlePathBrowseButtonClicked()
{
	IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
	if (DesktopPlatform)
	{
		FString FolderName;
		const FString Title = LOCTEXT("NewProjectBrowseTitle", "Choose a project location").ToString();
		const bool bFolderSelected = DesktopPlatform->OpenDirectoryDialog(
			FSlateApplication::Get().FindBestParentWindowHandleForDialogs(AsShared()),
			Title,
			LastBrowsePath,
			FolderName
		);

		if (bFolderSelected)
		{
			if (!FolderName.EndsWith(TEXT("/")))
			{
				FolderName += TEXT("/");
			}

			FPaths::MakePlatformFilename(FolderName);
			LastBrowsePath = FolderName;
			CurrentProjectFilePath = FolderName;
		}
	}

	return FReply::Handled();
}

bool SProjectDialog::IsCompilerRequired() const
{
	TSharedPtr<FTemplateItem> SelectedTemplate = GetSelectedTemplateItem();

	if (SelectedTemplate.IsValid())
	{
		return bShouldGenerateCode && !SelectedTemplate->CodeProjectFile.IsEmpty();
	}

	return false;
}

// Linux does not require an IDE to be setup to compile things
bool SProjectDialog::IsIDERequired() const
{
#if PLATFORM_LINUX
	return false;
#else
	return true;
#endif
}

EVisibility SProjectDialog::GetProjectSettingsVisibility() const
{
	const TArray<ETemplateSetting>& HiddenSettings = GetSelectedTemplateProperty(&FTemplateItem::HiddenSettings);
	return HiddenSettings.Contains(ETemplateSetting::All) ? EVisibility::Collapsed : EVisibility::Visible;
}

EVisibility SProjectDialog::GetSelectedTemplateClassVisibility() const
{
	return GetSelectedTemplateProperty(&FTemplateItem::ClassTypes).IsEmpty() == false ? EVisibility::Visible : EVisibility::Collapsed;
}

FText SProjectDialog::GetSelectedTemplateAssetTypes() const
{
	return FText::FromString(GetSelectedTemplateProperty(&FTemplateItem::AssetTypes));
}

FText SProjectDialog::GetSelectedTemplateClassTypes() const
{
	return FText::FromString(GetSelectedTemplateProperty(&FTemplateItem::ClassTypes));
}

EVisibility SProjectDialog::GetSelectedTemplateAssetVisibility() const
{
	return GetSelectedTemplateProperty(&FTemplateItem::AssetTypes).IsEmpty() == false ? EVisibility::Visible : EVisibility::Collapsed;
}

FString SProjectDialog::GetProjectFilenameWithPath() const
{
	if (CurrentProjectFilePath.IsEmpty())
	{
		// Don't even try to assemble the path or else it may be relative to the binaries folder!
		return TEXT("");
	}
	else
	{
		const FString ProjectName = CurrentProjectFileName;
		const FString ProjectPath = IFileManager::Get().ConvertToAbsolutePathForExternalAppForWrite(*CurrentProjectFilePath);
		const FString Filename = ProjectName + TEXT(".") + FProjectDescriptor::GetExtension();
		FString ProjectFilename = FPaths::Combine(*ProjectPath, *ProjectName, *Filename);
		FPaths::MakePlatformFilename(ProjectFilename);
		return ProjectFilename;
	}
}

void SProjectDialog::UpdateProjectFileValidity()
{
	// Global validity
	{
		bLastGlobalValidityCheckSuccessful = true;

		
		if (TSharedPtr<FTemplateItem> SelectedTemplate = GetSelectedTemplateItem(); !SelectedTemplate.IsValid())
		{
			bLastGlobalValidityCheckSuccessful = false;
			LastGlobalValidityErrorText = LOCTEXT("NoTemplateSelected", "No Template Selected");
		}
		else
		{
			if (IsCompilerRequired())
			{
				if (!FSourceCodeNavigation::IsCompilerAvailable())
				{
					bLastGlobalValidityCheckSuccessful = false;

					if (IsIDERequired())
					{
						LastGlobalValidityErrorText = FText::Format(LOCTEXT("NoCompilerFoundProjectDialog", "No compiler was found. In order to use a C++ template, you must first install {0}."), FSourceCodeNavigation::GetSuggestedSourceCodeIDE());
					}
					else
					{
						LastGlobalValidityErrorText = FText::Format(LOCTEXT("MissingIDEProjectDialog", "Your IDE {0} is missing or incorrectly configured, please consider using {1}"),
			FSourceCodeNavigation::GetSelectedSourceCodeIDE(), FSourceCodeNavigation::GetSuggestedSourceCodeIDE());
					}
				}
				else if (!FDesktopPlatformModule::Get()->IsUnrealBuildToolAvailable())
				{
					bLastGlobalValidityCheckSuccessful = false;
					LastGlobalValidityErrorText = LOCTEXT("UBTNotFound", "Engine source code was not found. In order to use a C++ template, you must have engine source code in Engine/Source.");
				}
			}
		}
	}

	// Name and Location Validity
	{
		bLastNameAndLocationValidityCheckSuccessful = true;

		if (!FPlatformMisc::IsValidAbsolutePathFormat(CurrentProjectFilePath))
		{
			bLastNameAndLocationValidityCheckSuccessful = false;
			LastNameAndLocationValidityErrorText = LOCTEXT("InvalidFolderPath", "The folder path is invalid");
		}
		else
		{
			FText FailReason;
			if (!GameProjectUtils::IsValidProjectFileForCreation(GetProjectFilenameWithPath(), FailReason))
			{
				bLastNameAndLocationValidityCheckSuccessful = false;
				LastNameAndLocationValidityErrorText = FailReason;
			}
		}

		if (CurrentProjectFileName.Contains(TEXT("/")) || CurrentProjectFileName.Contains(TEXT("\\")))
		{
			bLastNameAndLocationValidityCheckSuccessful = false;
			LastNameAndLocationValidityErrorText = LOCTEXT("SlashOrBackslashInProjectName", "The project name may not contain a slash or backslash");
		}
		else
		{
			FText FailReason;
			if (!GameProjectUtils::IsValidProjectFileForCreation(GetProjectFilenameWithPath(), FailReason))
			{
				bLastNameAndLocationValidityCheckSuccessful = false;
				LastNameAndLocationValidityErrorText = FailReason;
			}
		}
	}
}

void SProjectDialog::UpdateTemplateTrustWarning()
{
	if (TSharedPtr<FTemplateItem> SelectedTemplate = GetSelectedTemplateItem(); SelectedTemplate.IsValid() && SelectedTemplate->PluginWeak != nullptr)
	{
		if (const TSharedPtr<IPlugin> Plugin = SelectedTemplate->PluginWeak.Pin(); !Plugin->IsEnabled())
		{
			DisplayWarning(LOCTEXT("WarnNotEnabledTrust", "This template comes from a plugin that is not enabled by default. Make sure you trust the authors."));
			return;
		}
	}

	PersistentGlobalWarningLabelText = FText::GetEmpty();
}

void SProjectDialog::OnMajorTemplateCategorySelectionChanged(TSharedPtr<FTemplateCategory> Item, ESelectInfo::Type SelectType)
{
	if(Item.IsValid())
	{
		CurrentCategory = Item;

		SetCurrentMajorCategory(Item->Key);

		TemplateAndRecentProjectsSwitcher->SetActiveWidgetIndex(0);
		PathAreaSwitcher->SetActiveWidgetIndex(0);
	}
}

TSharedRef<ITableRow> SProjectDialog::ConstructMajorCategoryTableRow(TSharedPtr<FTemplateCategory> Item, const TSharedRef<STableViewBase>& TableView)
{
	TWeakPtr<FTemplateCategory> CurrentItem = Item;

	TSharedRef<STableRow<TSharedPtr<FTemplateCategory>>> Row =
		SNew(STableRow<TSharedPtr<FTemplateCategory>>, TableView)
		.Style(FAppStyle::Get(), "ProjectBrowser.TableRow")
		.ShowSelection(false)
		.Padding(2.0f);
	TWeakPtr<STableRow<TSharedPtr<FTemplateCategory>>> RowWeakPtr = Row;

	Row->SetContent(
		SNew(SMajorCategoryTile, Item)
		.IsSelected_Lambda([CurrentItem, this]() { return CurrentItem == CurrentCategory; })
	);

	return Row;
}

#undef LOCTEXT_NAMESPACE
