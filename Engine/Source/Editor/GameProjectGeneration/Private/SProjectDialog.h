// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/BitArray.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "Containers/SparseArray.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "GameProjectUtils.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "Input/Reply.h"
#include "Internationalization/Text.h"
#include "Layout/Visibility.h"
#include "Misc/Optional.h"
#include "Styling/SlateTypes.h"
#include "TemplateItem.h"
#include "Templates/SharedPointer.h"
#include "Templates/TypeHash.h"
#include "Templates/UniquePtr.h"
#include "Templates/UnrealTemplate.h"
#include "Types/SlateConstants.h"
#include "Types/SlateEnums.h"
#include "UObject/NameTypes.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STileView.h"

class ITableRow;
class SProjectBrowser;
class SWidget;
class SWidgetSwitcher;
struct FSlateBrush;
struct FTemplateCategory;

enum class EHardwareClass : uint8;
enum class EGraphicsPreset : uint8;

enum class EProjectDialogModeMode : uint8
{
	/** Shows new project templates and existing projects */
	Hybrid,
	/** Shows existing projects */
	OpenProject,
	/** Shows new project templates */
	NewProject,
};

class SProjectDialog : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SProjectDialog) {}
	SLATE_END_ARGS()

	/** Constructs this widget with InArgs */
	void Construct(const FArguments& InArgs, EProjectDialogModeMode Mode);

	~SProjectDialog();

	/** @returns All templates found on disk */
	static TMap<FName, TArray<TSharedPtr<FTemplateItem>>> FindTemplateProjects();

private:
	/** Accessor for the currently selected template item */
	TSharedPtr<FTemplateItem> GetSelectedTemplateItem() const;
	
	/** Accessor for the TemplateDefs of the currently selected template item. Will return either the BP or CPP version based on bShouldGenerateCode */
	UTemplateProjectDefs* GetSelectedTemplateDefs() const;
	/** Accessor for the Variant matching the SelectedVariantName for the currently selected TemplateDefs (see GetSelectedTemplateDefs) */
	const FTemplateVariant* GetSelectedTemplateDefsVariant() const
	{
		return GetSelectedTemplateDefsVariant(SelectedVariantName);
	}
	/** Accessor for the Variant with the given name for the currently selected TemplateDefs (see GetSelectedTemplateDefs). Returns null if not found */
	const FTemplateVariant* GetSelectedTemplateDefsVariant(FName VariantName) const;

	/** Helper function to allow direct lookup of the selected item's properties on a delegate */
	/** TRet should be defaulted but VS2012 doesn't allow default template arguments on non-class templates */
	template<typename T>
	T GetSelectedTemplateProperty(T FTemplateItem::* Prop) const
	{
		TSharedPtr<FTemplateItem> SelectedItem = GetSelectedTemplateItem();
		if (SelectedItem.IsValid())
		{
			return (*SelectedItem).*Prop;
		}

		return T();
	}

	void OnMajorTemplateCategorySelectionChanged(TSharedPtr<FTemplateCategory> Item, ESelectInfo::Type SelectType);
	TSharedRef<ITableRow> ConstructMajorCategoryTableRow(TSharedPtr<FTemplateCategory> Item, const TSharedRef<STableViewBase>& TableView);

	void PopulateTemplateCategories();

	/** Handler for when the project path string was changed */
	void OnCurrentProjectFilePathChanged(const FText& InValue);

	/** Handler for when the project name string was changed */
	void OnCurrentProjectFileNameChanged(const FText& InValue);

	FReply HandlePathBrowseButtonClicked();

	/** Checks the current project path an name for validity and updates cached values accordingly */
	void UpdateProjectFileValidity();

	/** Checks if the currently selected template is from a plugin that is not enabled and updates the warning message accordingly */
	void UpdateTemplateTrustWarning();

	/** Returns true if we have a code template selected */
	bool IsCompilerRequired() const;

	bool IsIDERequired() const;

	EVisibility GetProjectSettingsVisibility() const;

	/** Get a visibility of the class types display. If the string is empty this return Collapsed otherwise it will return Visible */
	EVisibility GetSelectedTemplateClassVisibility() const;

	/** Get a visibility of the asset types display. If the string is empty this return Collapsed otherwise it will return Visible */
	EVisibility GetSelectedTemplateAssetVisibility() const;

	EVisibility GetGlobalErrorVisibility() const { return GetGlobalErrorLabelText().IsEmpty() ? EVisibility::Hidden : EVisibility::HitTestInvisible; }

	EVisibility GetGlobalWarningVisibility() const { return PersistentGlobalWarningLabelText.IsEmpty() ? EVisibility::Hidden : EVisibility::HitTestInvisible; }

	EVisibility GetCreateButtonVisibility() const;

	EVisibility GetSuggestedIDEButtonVisibility() const;

	EVisibility GetDisableIDEButtonVisibility() const;

	/** Accessor for the project name text */
	FText GetCurrentProjectFileName() const;

	/** Accessor for the project path text */
	FText GetCurrentProjectFilePath() const;

	/** Get a visibility of the asset types display. If the string is empty this return Collapsed otherwise it will return Visible */
	FText GetSelectedTemplateAssetTypes() const;

	/** Get a string that details the class types referenced in the selected template */
	FText GetSelectedTemplateClassTypes() const;

	FText GetGlobalErrorLabelText() const;

	FText GetGlobalWarningLabelText() const;

	FText GetNameAndLocationValidityErrorText() const;

	/** Gets the assembled project filename with path */
	FString GetProjectFilenameWithPath() const;

	TSharedRef<SWidget> MakeNewProjectDialogButtons();

	TSharedRef<SWidget> MakeOpenProjectDialogButtons();

	TSharedRef<SWidget> MakeTemplateProjectView();

	TSharedRef<SWidget> MakeRecentProjectsView();

	TSharedRef<SWidget> MakeHybridView(EProjectDialogModeMode Mode);

	TSharedRef<SWidget> MakeProjectOptionsWidget();

	TSharedRef<SWidget> MakePluginInformationWidget();

	TSharedRef<SWidget> MakeRecentProjectsTile();

	TSharedRef<SWidget> MakeNewProjectPathArea();

	TSharedRef<SWidget> MakeOpenProjectPathArea();

	/** Get the images for the selected template preview and category */
	const FSlateBrush* GetSelectedTemplatePreviewImage() const;

	bool CanCreateProject() const;

	FReply OnCancel() const;
	
	int32 OnGetBlueprintOrCppIndex() const { return bShouldGenerateCode ? 1 : 0; }
	void OnSetBlueprintOrCppIndex(int32 Index);

	void SetHardwareClassTarget(EHardwareClass InHardwareClass);
	EHardwareClass GetHardwareClassTarget() const { return SelectedHardwareClassTarget; }

	void SetGraphicsPreset(EGraphicsPreset InGraphicsPreset);
	EGraphicsPreset GetGraphicsPreset() const { return SelectedGraphicsPreset; }

	void ResetVariants();
	EVisibility GetVariantsVisibility() const;
	TSharedRef< SWidget > GetVariantsDropdownContent();
	FText GetVariantsButtonText() const;
	FText GetVariantsButtonTooltip() const;

	/** Handler for when the selection changes in the template list */
	void HandleTemplateListViewSelectionChanged(TSharedPtr<FTemplateItem> TemplateItem, ESelectInfo::Type SelectInfo);

	void SetDefaultProjectLocation();

	void SetCurrentMajorCategory(FName Category);

	FReply OnRecentProjectsClicked();

	/** Called from the editor setting when bLoadTheMostRecentlyLoadedProjectAtStartup change to sync it */
	void OnLoadMostRecentProjectEditorSettingChanged(bool bInAutoLoadOption);

	/** Checked if it should load the most recent project, unchecked otherwise */
	ECheckBoxState IsLoadMostRecentProjectChecked() const;

	FProjectInformation CreateProjectInfo() const;
	bool CreateProject(const FString& ProjectFile);
	void CreateAndOpenProject();
	bool OpenProject(const FString& ProjectFile);
	bool OpenCodeIDE(const FString& ProjectFile);
	void CloseWindowIfAppropriate(bool ForceClose = false);
	void DisplayError(const FText& ErrorText);
	void DisplayWarning(const FText& WarningText);


	static TArray<TSharedPtr<FTemplateCategory>> GetAllTemplateCategories();
	static TSharedRef<FSlateBrush> SetupThumbnailBrush(TSharedRef<FSlateBrush> ThumbnailBrush);

private:
	TArray<TSharedPtr<FTemplateCategory>> TemplateCategories;
	TSharedPtr<FTemplateCategory> CurrentCategory;
	TSharedPtr<STileView<TSharedPtr<FTemplateCategory>>> MajorCategoryList;

	TSharedPtr<FTemplateCategory> RecentProjectsCategory;

	/** A map of category name to array of templates available for that category */
	TMap<FName, TArray<TSharedPtr<FTemplateItem>> > Templates;

	/** The filtered array of templates we are currently showing */
	TArray<TSharedPtr<FTemplateItem> > FilteredTemplateList;

	/** The slate widget representing the list of templates */
	TSharedPtr<STileView<TSharedPtr<FTemplateItem>>> TemplateListView;

	TSharedPtr<SWidgetSwitcher> TemplateAndRecentProjectsSwitcher;
	TSharedPtr<SWidgetSwitcher> PathAreaSwitcher;

	TSharedPtr<SProjectBrowser> ProjectBrowser;

	TSharedPtr<FSlateDynamicImageBrush> PluginIconDynamicImageBrush;

	FString LastBrowsePath;
	FString CurrentProjectFileName;
	FString CurrentProjectFilePath;

	FText PersistentGlobalErrorLabelText;
	FText PersistentGlobalWarningLabelText;

	/** The global error text from the last validity check */
	FText LastGlobalValidityErrorText;

	FText LastNameAndLocationValidityErrorText;

	/** Name of the currently selected template category */
	FName ActiveCategory;

	EHardwareClass SelectedHardwareClassTarget;

	EGraphicsPreset SelectedGraphicsPreset;

	FName SelectedVariantName;
	TArray<FName> TemplateVariantNames;

	TSharedPtr<FSlateBrush> RecentProjectsBrush;

	static TSharedPtr<FSlateBrush> CustomTemplateBrush;

	SVerticalBox::FSlot* ProjectOptionsSlot;

	SVerticalBox::FSlot* PluginInformationSlot;

	/** Whether or not to enable XR in the created project. */
	bool bEnableXR;

	/** Whether or not we should use the blueprint or C++ version of this template. */
	bool bShouldGenerateCode;

	/** True if the last global validity check returned that the project path is valid for creation */
	bool bLastGlobalValidityCheckSuccessful;

	/** True if the last NameAndLocation validity check returned that the project path is valid for creation */
	bool bLastNameAndLocationValidityCheckSuccessful;

	/** Synced setting to the EditorSetting bLoadTheMostRecentlyLoadedProjectAtStartup */
	bool bLoadMostRecentlyLoadedProject = false;
};
