// Copyright Epic Games, Inc. All Rights Reserved.

#include "SVerbChoiceDialog.h"

#include "Containers/UnrealString.h"
#include "Editor.h"
#include "Editor/EditorEngine.h"
#include "Fonts/SlateFontInfo.h"
#include "HAL/PlatformApplicationMisc.h"
#include "HAL/PlatformMisc.h"
#include "Input/Events.h"
#include "InputCoreTypes.h"
#include "Internationalization/Internationalization.h"
#include "Layout/BasicLayoutWidgetSlot.h"
#include "Layout/Children.h"
#include "Layout/Margin.h"
#include "SlateOptMacros.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Types/SlateEnums.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SHyperlink.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SWindow.h"
#include "Widgets/Text/STextBlock.h"

struct FGeometry;

int32 SVerbChoiceDialog::ShowModal(const FText& InTitle, const FText& InText, const TArray<FText>& InButtons)
{
	TArray<FText> Hyperlinks;
	return ShowModal(InTitle, InText, Hyperlinks, InButtons);
}

int32 SVerbChoiceDialog::ShowModal( const FText& InTitle, const FText& InMessage, const TArray<FText>& InHyperlinks, const TArray<FText>& InButtons)
{
	TSharedRef<SWindow> ModalWindow = SNew(SWindow)
		.Title( InTitle )
		.SizingRule( ESizingRule::Autosized )
		.AutoCenter(EAutoCenter::PreferredWorkArea)
		.HasCloseButton(false)
		.SupportsMinimize(false) 
		.SupportsMaximize(false);

	TSharedRef<SVerbChoiceDialog> MessageBox = SNew(SVerbChoiceDialog)
		.ParentWindow(ModalWindow)
		.Message( InMessage )
		.Hyperlinks( InHyperlinks )
		.Buttons( InButtons )
		.WrapMessageAt(640.0f);

	ModalWindow->SetContent( MessageBox );

	GEditor->EditorAddModalWindow(ModalWindow);

	return MessageBox->Response;
}

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void SVerbChoiceDialog::Construct( const FArguments& InArgs )
{
	ParentWindow = InArgs._ParentWindow.Get();
	ParentWindow->SetWidgetToFocusOnActivate(SharedThis(this));
	Response = EAppReturnType::Cancel;

	FSlateFontInfo MessageFont( FAppStyle::GetFontStyle("StandardDialog.LargeFont"));
	Message = InArgs._Message;
	Hyperlinks = InArgs._Hyperlinks;
	Buttons = InArgs._Buttons;
	
	TSharedPtr<SUniformGridPanel> ButtonBox;
	TSharedPtr<SHorizontalBox> HyperlinksBox;

	this->ChildSlot
		[	
			SNew(SBorder)
				.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
				[
					SNew(SVerticalBox)

					+ SVerticalBox::Slot()
						.HAlign(HAlign_Fill)
						.VAlign(VAlign_Fill)
						.FillHeight(1.0f)
						.MaxHeight(550)
						.Padding(12.0f)
						[
							SNew(SScrollBox)

							+ SScrollBox::Slot()
								[
									SNew(STextBlock)
										.Text(Message)
										.Font(MessageFont)
										.WrapTextAt(InArgs._WrapMessageAt)
								]
						]

					+SVerticalBox::Slot()
						.AutoHeight()
						.Padding(0.0f)
						[
							SNew(SHorizontalBox)

							+SHorizontalBox::Slot()
								.AutoWidth()
								.HAlign(HAlign_Left)
								.VAlign(VAlign_Center)
								.Padding(FMargin(12.0f, 0.f, 9.0f, 0.f))
								[
									SNew(SButton)
									.ButtonStyle(FAppStyle::Get(), "SimpleButton")
									.OnClicked(this, &SVerbChoiceDialog::HandleCopyMessageButtonClicked)
									.ToolTipText(NSLOCTEXT("SVerbChoiceDialog", "CopyMessageTooltip", "Copy the text in this message to the clipboard (CTRL+C)"))
									.ContentPadding(2.f)
									.Content()
									[
										SNew(SImage)
										.Image(FAppStyle::Get().GetBrush("Icons.Clipboard"))
										.ColorAndOpacity(FSlateColor::UseForeground())
									]
								]

							+ SHorizontalBox::Slot()
								.FillWidth(1.0f)
								.HAlign(HAlign_Left)
								.VAlign(VAlign_Center)
								[
									SAssignNew( HyperlinksBox, SHorizontalBox )
								]

							+ SHorizontalBox::Slot()
								.AutoWidth()
								.HAlign(HAlign_Right)
								.VAlign(VAlign_Center)
								.Padding(5.0f)
								[
									SAssignNew( ButtonBox, SUniformGridPanel )
										.SlotPadding(FAppStyle::GetMargin("StandardDialog.SlotPadding"))
										.MinDesiredSlotWidth(FAppStyle::GetFloat("StandardDialog.MinDesiredSlotWidth"))
										.MinDesiredSlotHeight(FAppStyle::GetFloat("StandardDialog.MinDesiredSlotHeight"))
								]
						]
				]
		];

	for(int32 Idx = 0; Idx < Hyperlinks.Get().Num(); Idx++)
	{
		HyperlinksBox->AddSlot()
			.AutoWidth()
			.HAlign(HAlign_Left)
			.Padding(FAppStyle::GetMargin("StandardDialog.SlotPadding"))
			[
				SNew(SHyperlink)
					.Text( Hyperlinks.Get()[Idx] )
					.OnNavigate(this, &SVerbChoiceDialog::HandleHyperlinkClicked, ~Idx)
			];
	}

	for(int32 Idx = 0; Idx < Buttons.Get().Num(); Idx++)
	{
		ButtonBox->AddSlot(Idx, 0)
			[
				SNew( SButton )
				.Text( Buttons.Get()[Idx] )
				.OnClicked( this, &SVerbChoiceDialog::HandleButtonClicked, Idx )
				.ContentPadding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
				.HAlign(HAlign_Center)
			];
	}
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

FReply SVerbChoiceDialog::OnKeyDown( const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent )
{
	//see if we pressed the Enter or Spacebar keys
	if( InKeyEvent.GetKey() == EKeys::Escape )
	{
		return HandleButtonClicked(EAppReturnType::Cancel);
	}

	if (InKeyEvent.GetKey() == EKeys::C && InKeyEvent.IsControlDown())
	{
		CopyMessageToClipboard();

		return FReply::Handled();
	}

	//if it was some other button, ignore it
	return FReply::Unhandled();
}

bool SVerbChoiceDialog::SupportsKeyboardFocus() const
{
	return true;
}

void SVerbChoiceDialog::CopyMessageToClipboard( )
{
	FPlatformApplicationMisc::ClipboardCopy( *Message.Get().ToString() );
}

FReply SVerbChoiceDialog::HandleCopyMessageButtonClicked( )
{
	CopyMessageToClipboard();
	return FReply::Handled();
}

void SVerbChoiceDialog::HandleHyperlinkClicked( int32 InResponse )
{
	Response = InResponse;
	ParentWindow->RequestDestroyWindow();
}

FReply SVerbChoiceDialog::HandleButtonClicked( int32 InResponse )
{
	Response = InResponse;
	ParentWindow->RequestDestroyWindow();

	return FReply::Handled();
}
