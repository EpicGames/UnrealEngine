// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "HAL/Platform.h"
#include "Input/Reply.h"
#include "Internationalization/Text.h"
#include "Misc/Attribute.h"
#include "Templates/SharedPointer.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class SWindow;
struct FGeometry;
struct FKeyEvent;

class SVerbChoiceDialog : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS( SVerbChoiceDialog )	{}
		SLATE_ATTRIBUTE(TSharedPtr<SWindow>, ParentWindow)
		SLATE_ATTRIBUTE(FText, Message)	
		SLATE_ATTRIBUTE(TArray<FText>, Hyperlinks)
		SLATE_ATTRIBUTE(TArray<FText>, Buttons)
		SLATE_ATTRIBUTE(float, WrapMessageAt)
	SLATE_END_ARGS()

	/** Displays the modal dialog box */
	static int32 ShowModal(const FText& InTitle, const FText& InText, const TArray<FText>& InButtons);

	/** Displays the modal dialog box, also allowing customization of the hyperlinks */
	static int32 ShowModal(const FText& InTitle, const FText& InText, const TArray<FText>& InHyperlinks, const TArray<FText>& InButtons);

	/** Constructs the dialog */
	void Construct( const FArguments& InArgs );

	/** Override behavior when a key is pressed */
	virtual	FReply OnKeyDown( const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent ) override;

	/** Override the base method to allow for keyboard focus */
	virtual bool SupportsKeyboardFocus() const override;

protected:
	/** Copies the message text to the clipboard. */
	void CopyMessageToClipboard( );

private:
	/** Handles clicking the 'Copy Message' button. */
	FReply HandleCopyMessageButtonClicked( );

	/** Handles clicking on a hyperlink. */
	void HandleHyperlinkClicked( int32 InResponse );

	/** Handles clicking a message box button. */
	FReply HandleButtonClicked( int32 InResponse );

	int32 Response;
	TSharedPtr<SWindow> ParentWindow;
	TAttribute<FText> Message;
	TAttribute< TArray<FText> > Buttons;
	TAttribute< TArray<FText> > Hyperlinks;
};
