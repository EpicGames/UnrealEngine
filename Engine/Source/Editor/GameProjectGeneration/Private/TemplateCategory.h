// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "TemplateProjectDefs.h"

struct FSlateBrush;

struct FTemplateCategory
{
	/** The overriden name of this category */
	FText DisplayNameOverride;
	
	/** The overriden description of the templates contained within this category */
	FText DescriptionOverride;

	TArray<FLocalizedTemplateString> LocalizedDisplayNames;
	TArray<FLocalizedTemplateString> LocalizedDescriptions;

	/** A thumbnail to help identify this category (on the tab) */
	TSharedPtr<FSlateBrush> Icon;

	/** A unique key for this category */
	FName Key;

	/** Is this an enterprise project? Will end up including Datasmith modules if true. */
	bool IsEnterprise = false;

	/** Localized name of this category */
	FText GetDisplayName() const
	{
		if (!DisplayNameOverride.IsEmpty())
		{
			return DisplayNameOverride;
		}
		return FLocalizedTemplateString::GetLocalizedText(LocalizedDisplayNames);
	}
	/** A description of the templates contained within this category */
	FText GetDescription() const
	{
		if (!DescriptionOverride.IsEmpty())
		{
			return DescriptionOverride;
		}
		return FLocalizedTemplateString::GetLocalizedText(LocalizedDescriptions);
	}
};
