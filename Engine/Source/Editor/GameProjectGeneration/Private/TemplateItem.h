// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Styling/SlateBrush.h"
#include "TemplateProjectDefs.h"
#include "Interfaces/IPluginManager.h"

/** Struct describing a single template project */
struct FTemplateItem
{
	FText		DisplayNameOverride;
	FText		DescriptionOverride;
	TArray<FLocalizedTemplateString> LocalizedDisplayNames;
	TArray<FLocalizedTemplateString> LocalizedDescriptions;
	TArray<FName> Categories;

	FString		Key;
	FString		SortKey;

	TSharedPtr<FSlateBrush> Thumbnail;
	TSharedPtr<FSlateBrush> PreviewImage;
	TWeakPtr<IPlugin> PluginWeak;


	FString		ClassTypes;
	FString		AssetTypes;

	FString		CodeProjectFile;
	UTemplateProjectDefs* CodeTemplateDefs = nullptr;
	FString		BlueprintProjectFile;
	UTemplateProjectDefs* BlueprintTemplateDefs = nullptr;

	TArray<ETemplateSetting> HiddenSettings;

	bool		bIsEnterprise = false;
	bool		bIsBlankTemplate = false;
	bool		bThumbnailAsIcon = false;
	
	/** Localized name of this template */
	FText GetDisplayName() const
	{
		if (!DisplayNameOverride.IsEmpty())
		{
			return DisplayNameOverride;
		}
		const FText Name = FLocalizedTemplateString::GetLocalizedText(LocalizedDisplayNames);
		return Name.IsEmpty() ? FText::FromString(Key) : Name;
	}
	/** A description of the template */
	FText GetDescription() const
	{
		if (!DescriptionOverride.IsEmpty())
		{
			return DescriptionOverride;
		}
		return FLocalizedTemplateString::GetLocalizedText(LocalizedDescriptions);
	}
};
