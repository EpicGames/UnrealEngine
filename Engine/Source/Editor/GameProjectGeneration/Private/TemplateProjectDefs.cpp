// Copyright Epic Games, Inc. All Rights Reserved.


#include "TemplateProjectDefs.h"

#include "Algo/Find.h"
#include "HAL/Platform.h"
#include "Internationalization/Culture.h"
#include "Internationalization/Internationalization.h"
#include "Misc/CString.h"
#include "Templates/SharedPointer.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TemplateProjectDefs)

FText FLocalizedTemplateString::GetLocalizedText(const TArray<FLocalizedTemplateString>& LocalizedStrings)
{
	auto FindLocalizedStringForCulture = [&LocalizedStrings](const FString& InCulture)
	{
		return LocalizedStrings.FindByPredicate([&InCulture](const FLocalizedTemplateString& LocalizedString)
		{
			return InCulture == LocalizedString.Language;
		});
	};

	const FString CurrentLanguage = FInternationalization::Get().GetCurrentLanguage()->GetName();

	// Try and find a prioritized localized translation
	{
		const TArray<FString> PrioritizedCultureNames = FInternationalization::Get().GetPrioritizedCultureNames(CurrentLanguage);
		for (const FString& CultureName : PrioritizedCultureNames)
		{
			if (const FLocalizedTemplateString* LocalizedStringForCulture = FindLocalizedStringForCulture(CultureName))
			{
				return FText::FromString(LocalizedStringForCulture->Text);
			}
		}
	}

	// We failed to find a localized translation, see if we have English text available to use
	if (CurrentLanguage != TEXT("en"))
	{
		if (const FLocalizedTemplateString* LocalizedStringForEnglish = FindLocalizedStringForCulture(TEXT("en")))
		{
			return FText::FromString(LocalizedStringForEnglish->Text);
		}
	}

	// We failed to find English, see if we have any translations available to use
	if (LocalizedStrings.Num() > 0)
	{
		return FText::FromString(LocalizedStrings[0].Text);
	}

	return FText();
}

FText FTemplateVariant::GetDisplayNameText() const
{
	return FLocalizedTemplateString::GetLocalizedText(LocalizedDisplayNames);
}

FText FTemplateVariant::GetLocalizedDescription() const
{
	return FLocalizedTemplateString::GetLocalizedText(LocalizedDescriptions);
}

FTemplateConfigValue::FTemplateConfigValue(const FString& InFile, const FString& InSection, const FString& InKey, const FString& InValue, bool InShouldReplaceExistingValue)
	: ConfigFile(InFile)
	, ConfigSection(InSection)
	, ConfigKey(InKey)
	, ConfigValue(InValue)
	, bShouldReplaceExistingValue(InShouldReplaceExistingValue)
{
}


UTemplateProjectDefs::UTemplateProjectDefs(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bAllowProjectCreation = true;
	bThumbnailAsIcon = false;
	EditDetailLevelPreference = EFeaturePackDetailLevel::Standard;
}

void UTemplateProjectDefs::FixupStrings(const FString& TemplateName, const FString& ProjectName)
{
	for ( auto IgnoreIt = FoldersToIgnore.CreateIterator(); IgnoreIt; ++IgnoreIt )
	{
		FixString(*IgnoreIt, TemplateName, ProjectName);
	}

	for ( auto IgnoreIt = FilesToIgnore.CreateIterator(); IgnoreIt; ++IgnoreIt )
	{
		FixString(*IgnoreIt, TemplateName, ProjectName);
	}

	for ( auto RenameIt = FolderRenames.CreateIterator(); RenameIt; ++RenameIt )
	{
		FTemplateFolderRename& FolderRename = *RenameIt;
		FixString(FolderRename.From, TemplateName, ProjectName);
		FixString(FolderRename.To, TemplateName, ProjectName);
	}

	for ( auto ReplacementsIt = FilenameReplacements.CreateIterator(); ReplacementsIt; ++ReplacementsIt )
	{
		FTemplateReplacement& Replacement = *ReplacementsIt;
		FixString(Replacement.From, TemplateName, ProjectName);
		FixString(Replacement.To, TemplateName, ProjectName);
	}

	for ( auto ReplacementsIt = ReplacementsInFiles.CreateIterator(); ReplacementsIt; ++ReplacementsIt )
	{
		FTemplateReplacement& Replacement = *ReplacementsIt;
		FixString(Replacement.From, TemplateName, ProjectName);
		FixString(Replacement.To, TemplateName, ProjectName);
	}
}

FText UTemplateProjectDefs::GetDisplayNameText()
{
	return FLocalizedTemplateString::GetLocalizedText(LocalizedDisplayNames);
}

FText UTemplateProjectDefs::GetLocalizedDescription()
{
	return FLocalizedTemplateString::GetLocalizedText(LocalizedDescriptions);
}

const FTemplateVariant* UTemplateProjectDefs::FindVariant(FName VariantName) const
{
	return VariantName == NAME_None ? nullptr : Algo::FindByPredicate(Variants, [&](const FTemplateVariant& Variant)
	{
		return Variant.Name == VariantName;
	});
}

void UTemplateProjectDefs::FixString(FString& InOutStringToFix, const FString& TemplateName, const FString& ProjectName)
{
	InOutStringToFix.ReplaceInline(TEXT("%TEMPLATENAME%"), *TemplateName, ESearchCase::CaseSensitive);
	InOutStringToFix.ReplaceInline(TEXT("%TEMPLATENAME_UPPERCASE%"), *TemplateName.ToUpper(), ESearchCase::CaseSensitive);
	InOutStringToFix.ReplaceInline(TEXT("%TEMPLATENAME_LOWERCASE%"), *TemplateName.ToLower(), ESearchCase::CaseSensitive);

	InOutStringToFix.ReplaceInline(TEXT("%PROJECTNAME%"), *ProjectName, ESearchCase::CaseSensitive);
	InOutStringToFix.ReplaceInline(TEXT("%PROJECTNAME_UPPERCASE%"), *ProjectName.ToUpper(), ESearchCase::CaseSensitive);
	InOutStringToFix.ReplaceInline(TEXT("%PROJECTNAME_LOWERCASE%"), *ProjectName.ToLower(), ESearchCase::CaseSensitive);
}
