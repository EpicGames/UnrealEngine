// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "HAL/FileManager.h"
#include "Misc/Paths.h"
#include "Misc/AutomationTest.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Class.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Builders/CubeBuilder.h"
#include "GameFramework/PlayerStart.h"
#include "Editor.h"
#include "LevelEditorViewport.h"
#include "FileHelpers.h"
#include "ProjectDescriptor.h"
#include "GameProjectUtils.h"
#include "SProjectDialog.h"

#include "DesktopPlatformModule.h"
#include "Tests/AutomationTestSettings.h"
#include "Tests/AutomationEditorCommon.h"
#include "TemplateCategory.h"
#include "TemplateItem.h"

#if WITH_DEV_AUTOMATION_TESTS

DEFINE_LOG_CATEGORY_STATIC(LogGameProjectGenerationTests, Log, All);

namespace GameProjectAutomationUtils
{
	/**
	* Generates the desired project file name
	*/
	static FString GetDesiredProjectFilename()
	{
		UAutomationTestSettings const* AutomationTestSettings = GetDefault<UAutomationTestSettings>();
		check(AutomationTestSettings);

		FString ProjectName;
		const FString ProjectNameOverride = AutomationTestSettings->BuildPromotionTest.NewProjectSettings.NewProjectNameOverride;
		if (ProjectNameOverride.Len())
		{
			ProjectName = ProjectNameOverride;
		}
		else
		{
			ProjectName = TEXT("NewTestProject");
		}

		FString ProjectPath;
		const FString ProjectPathOverride = AutomationTestSettings->BuildPromotionTest.NewProjectSettings.NewProjectFolderOverride.Path;
		if (ProjectPathOverride.Len())
		{
			ProjectPath = ProjectPathOverride;
		}
		else
		{
			ProjectPath = IFileManager::Get().ConvertToAbsolutePathForExternalAppForWrite(*FAutomationTestFramework::Get().GetUserAutomationDirectory());
		}


		const FString Filename = ProjectName + TEXT(".") + FProjectDescriptor::GetExtension();
		FString ProjectFilename = FPaths::Combine(*ProjectPath, *ProjectName, *Filename);
		FPaths::MakePlatformFilename(ProjectFilename);

		return ProjectFilename;
	}


	/* 
	 * Create a project from a template with a given criteria
	 * @param	InTemplates			List of available project templates
	 * @param	InTargetedHardware	Target hardware (EHardwareClass)
	 * @param	InGraphicPreset		Graphics preset (EGraphicsPreset)
	 * @param	InCategory			Target category (EContentSourceCategory)
	 * @param	OutMatchedProjects	Total projects matching criteria
	 * @param	OutCreatedProjects	Total projects succesfully created
	 */
	static void CreateProjectSet(TMap<FName, TArray<TSharedPtr<FTemplateItem>> >& InTemplates, EHardwareClass InTargetedHardware, EGraphicsPreset InGraphicPreset, FName InCategory, int32 &OutCreatedProjects, int32 &OutMatchedProjects)
	{		
		// If this is empty, it will use the same name for each project, otherwise it will create a project based on target platform and source template
		FString TestRootFolder;// = "ProjectTests";
		
		// This has the code remove the projects once created
		bool bRemoveCreatedProjects = true;		
		OutCreatedProjects = 0;
		OutMatchedProjects = 0;

		// Iterate all templates and try to create those that match the required criteria
		for (auto& EachTemplate : InTemplates)
		{
			// Check this is the correct category
			if (EachTemplate.Key == InCategory)
			{
				// Now iterate each template in the category
				for (TSharedPtr<FTemplateItem> OneTemplate : EachTemplate.Value)
				{
					TSharedPtr<FTemplateItem> Item = OneTemplate;

					// loop over the code project file and the blueprint file
					FString ProjectFile = Item->BlueprintProjectFile;
					for (int i = 0; i < 2; ++i)
					{
						if (ProjectFile.IsEmpty())
							continue;

						FString DesiredProjectFilename;
						if (TestRootFolder.IsEmpty() == true)
						{
							// Same name for all
							DesiredProjectFilename = GameProjectAutomationUtils::GetDesiredProjectFilename();
						}
						else
						{
							// Unique names
							FString Hardware;
							if (InTargetedHardware == EHardwareClass::Desktop)
							{
								Hardware = TEXT("Dsk");
							}
							else
							{
								Hardware = TEXT("Mob");
							}
							FString ProjectName = FPaths::GetCleanFilename(ProjectFile).Replace(TEXT("TP_"), TEXT(""));
							FString ProjectDirName = FPaths::GetBaseFilename(ProjectFile).Replace(TEXT("TP_"), TEXT(""));
							FString BasePath = FPaths::RootDir();
							DesiredProjectFilename = FString::Printf(TEXT("%s/%s/%s%s/%s%s"), *BasePath, *TestRootFolder, *Hardware, *ProjectDirName, *Hardware, *ProjectName);
						}

						// If the project already exists, delete it just in case things were left in a bad state.
						if (IFileManager::Get().DirectoryExists(*FPaths::GetPath(DesiredProjectFilename)))
						{
							IFileManager::Get().DeleteDirectory(*FPaths::GetPath(DesiredProjectFilename), /*RequireExists=*/false, /*Tree=*/true);
						}

						// Setup creation parameters
						FText FailReason, FailLog;
						FProjectInformation ProjectInfo;
						ProjectInfo.ProjectFilename = DesiredProjectFilename;
						ProjectInfo.bShouldGenerateCode = ProjectFile == Item->CodeProjectFile;
						ProjectInfo.TemplateFile = ProjectFile;
						ProjectInfo.TargetedHardware = InTargetedHardware;
						ProjectInfo.DefaultGraphicsPerformance = InGraphicPreset;
						TArray<FString> CreatedFiles;
						OutMatchedProjects++;

						// Finally try to create the project					
						if (!GameProjectUtils::CreateProject(ProjectInfo, FailReason, FailLog, &CreatedFiles))
						{
							// Failed, report the reason
							UE_LOGF(LogGameProjectGenerationTests, Error, "Failed to create %ls project %ls based on %ls. Reason: %ls\nProject Creation Failure Log:\n%ls", *InCategory.ToString(), *DesiredProjectFilename, *Item->GetDisplayName().ToString(), *FailReason.ToString(), *FailLog.ToString());
						}
						else
						{
							// Created ok
							OutCreatedProjects++;

							// Now remove the files we just created (if required)
							if (bRemoveCreatedProjects == true) //-V547
							{
								FString RootFolder = FPaths::GetPath(DesiredProjectFilename);
								GameProjectUtils::DeleteCreatedFiles(RootFolder, CreatedFiles);
							}
						}
					}

					ProjectFile = Item->CodeProjectFile;
				}
			}
		}
		return;
	}
}

/**
* Automation test to clean up old test project files
*/
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FBuildPromotionNewProjectCleanupTest, "System.Promotion.Project Promotion Pass.Step 1 Blank Project Creation.Cleanup Potential Project Location", /*EAutomationTestFlags::Disabled |*/ EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter );
bool FBuildPromotionNewProjectCleanupTest::RunTest(const FString& Parameters)
{
	FString DesiredProjectFilename = GameProjectAutomationUtils::GetDesiredProjectFilename();

	if (FPaths::FileExists(DesiredProjectFilename))
	{
		UE_LOGF(LogGameProjectGenerationTests, Display, "Found an old project file at %ls", *DesiredProjectFilename);
		if (FPaths::IsProjectFilePathSet())
		{
			FString CurrentProjectPath = FPaths::GetProjectFilePath();
			if (CurrentProjectPath == DesiredProjectFilename)
			{
				UE_LOGF(LogGameProjectGenerationTests, Warning, "Can not clean up the target project location because it is the current active project.");
			}
			else
			{
				UE_LOGF(LogGameProjectGenerationTests, Display, "Removing files from old project path: %ls", *FPaths::GetPath(DesiredProjectFilename));
				bool bEnsureExists = false;
				bool bDeleteEntireTree = true;
				IFileManager::Get().DeleteDirectory(*FPaths::GetPath(DesiredProjectFilename), bEnsureExists, bDeleteEntireTree);
			}
		}
	}
	else
	{
		UE_LOGF(LogGameProjectGenerationTests, Display, "Target project location is clear");
	}

	return true;
}

/**
* Automation test to create a new project
*/
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FBuildPromotionNewProjectCreateTest, "System.Promotion.Project Promotion Pass.Step 1 Blank Project Creation.Create Project", /*EAutomationTestFlags::Disabled |*/ EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter | EAutomationTestFlags::RequiresUser);
bool FBuildPromotionNewProjectCreateTest::RunTest(const FString& Parameters)
{
	FString DesiredProjectFilename = GameProjectAutomationUtils::GetDesiredProjectFilename();

	if (FPaths::FileExists(DesiredProjectFilename))
	{
		UE_LOGF(LogGameProjectGenerationTests, Warning, "A project already exists at the target location: %ls", *DesiredProjectFilename);
		const FString OldProjectFolder = FPaths::GetPath(DesiredProjectFilename);
		const FString OldProjectName = FPaths::GetBaseFilename(DesiredProjectFilename);
		const FString RootFolder = FPaths::GetPath(OldProjectFolder);

		//Add a number to the end
		for (uint32 i = 2;; ++i)
		{
			const FString PossibleProjectName = FString::Printf(TEXT("%s%i"), *OldProjectName, i);
			const FString PossibleProjectFilename = FString::Printf(TEXT("%s/%s/%s.%s"), *RootFolder, *PossibleProjectName, *PossibleProjectName, *FProjectDescriptor::GetExtension());
			if (!FPaths::FileExists(PossibleProjectFilename))
			{
				DesiredProjectFilename = PossibleProjectFilename;
				UE_LOGF(LogGameProjectGenerationTests, Warning, "Changing the target project name to: %ls", *FPaths::GetBaseFilename(DesiredProjectFilename));
				break;
			}
		}
	}

	FText FailReason, FailLog;
	FProjectInformation ProjectInfo;
	ProjectInfo.ProjectFilename = DesiredProjectFilename;
	if (GameProjectUtils::CreateProject(ProjectInfo, FailReason, FailLog))
	{
		UE_LOGF(LogGameProjectGenerationTests, Display, "Generated a new project: %ls", *DesiredProjectFilename);
		UE_LOGF(LogGameProjectGenerationTests, Display, "Test successful!");
		UE_LOGF(LogGameProjectGenerationTests, Display, "\nPlease switch to the new project and continue to Step 2.");
	}
	else
	{
		UE_LOGF(LogGameProjectGenerationTests, Error, "Could not generate new project: %ls - %ls", *FailReason.ToString(), *FailLog.ToString());
		UE_LOGF(LogGameProjectGenerationTests, Display, "Test failed!");
	}

	return true;
}

/**
* Automation test to create a simple level and save it
*/
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FBuildPromotionNewProjectMapTest, "System.Promotion.Project Promotion Pass.Step 2 Basic Level Creation.Create Basic Level", /*EAutomationTestFlags::Disabled |*/ EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter);
bool FBuildPromotionNewProjectMapTest::RunTest(const FString& Parameters)
{
	//New level
	UWorld* CurrentWorld = FAutomationEditorCommonUtils::CreateNewMap();
	if (!CurrentWorld)
	{
		UE_LOGF(LogGameProjectGenerationTests, Error, "Failed to create an empty level");
		return false;
	}

	UE_LOGF(LogGameProjectGenerationTests, Display, "Adding Level Geometry");

	//Add some bsp and a player start
	GEditor->Exec(CurrentWorld, TEXT("BRUSH Scale 1 1 1"));
	for(FLevelEditorViewportClient* ViewportClient : GEditor->GetLevelViewportClients())
	{
		if (!ViewportClient->IsOrtho())
		{
			ViewportClient->SetViewLocation(FVector(176, 2625, 2075));
			ViewportClient->SetViewRotation(FRotator(319, 269, 1));
		}
	}
	ULevel* CurrentLevel = CurrentWorld->GetCurrentLevel();

	//Cube Additive Brush
	UCubeBuilder* CubeAdditiveBrushBuilder = Cast<UCubeBuilder>(GEditor->FindBrushBuilder(UCubeBuilder::StaticClass()));
	CubeAdditiveBrushBuilder->X = 4096.0f;
	CubeAdditiveBrushBuilder->Y = 4096.0f;
	CubeAdditiveBrushBuilder->Z = 128.0f;
	CubeAdditiveBrushBuilder->Build(CurrentWorld);
	GEditor->Exec(CurrentWorld, TEXT("BRUSH MOVETO X=0 Y=0 Z=0"));
	GEditor->Exec(CurrentWorld, TEXT("BRUSH ADD"));

	//Add a playerstart
	const FTransform Transform(FRotator(-16384, 0, 0), FVector(0.f, 1750.f, 166.f));
	AActor* PlayerStart = GEditor->AddActor(CurrentWorld->GetCurrentLevel(), APlayerStart::StaticClass(), Transform);
	if (PlayerStart)
	{
		UE_LOGF(LogGameProjectGenerationTests, Display, "Added a player start");
	}
	else
	{
		UE_LOGF(LogGameProjectGenerationTests, Error, "Failed to add a player start");
	}

	// Save the map
    FString PathToSave = FPaths::ConvertRelativePathToFull(FPaths::ProjectContentDir() + TEXT("Maps/NewProjectTest.umap"));
	FEditorFileUtils::SaveLevel(CurrentLevel, PathToSave);
	UE_LOGF(LogGameProjectGenerationTests, Display, "Saved map");

	return true;
}

/*
* Template project creation test
*/
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FCreateBPTemplateProjectAutomationTests, "System.Promotion.Project Promotion Pass.Step 3 NewProjectCreationTests.CreateBlueprintProjects", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter /*| EAutomationTestFlags::Disabled*/)

/** 
 * Uses the new project wizard to locate all templates available for new blueprint project creation and verifies creation succeeds.
 *
 * @param Parameters - Unused for this test
 * @return	TRUE if the test was successful, FALSE otherwise
 */
bool FCreateBPTemplateProjectAutomationTests::RunTest(const FString& Parameters)
{
	TMap<FName, TArray<TSharedPtr<FTemplateItem>> > Templates = SProjectDialog::FindTemplateProjects();
	int32 OutMatchedProjectsDesk = 0;
	int32 OutCreatedProjectsDesk = 0;
	GameProjectAutomationUtils::CreateProjectSet(Templates, EHardwareClass::Desktop, EGraphicsPreset::Maximum, "Game", OutMatchedProjectsDesk, OutCreatedProjectsDesk);

	int32 OutMatchedProjectsMob = 0;
	int32 OutCreatedProjectsMob = 0;
	GameProjectAutomationUtils::CreateProjectSet(Templates, EHardwareClass::Mobile, EGraphicsPreset::Maximum, "Visualization", OutMatchedProjectsMob, OutCreatedProjectsMob);
	
	return ( OutMatchedProjectsDesk == OutCreatedProjectsDesk ) && ( OutMatchedProjectsMob == OutCreatedProjectsMob  );
}

/*
* Template project creation test
*/
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FCreateCPPTemplateProjectAutomationTests, "System.Promotion.Project Promotion Pass.Step 3 NewProjectCreationTests.CreateCodeProjects", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter /*| EAutomationTestFlags::Disabled*/)

/** 
 * Uses the new project wizard to locate all templates available for new code project creation and verifies creation succeeds.
 *
 * @param Parameters - Unused for this test
 * @return	TRUE if the test was successful, FALSE otherwise
 */
bool FCreateCPPTemplateProjectAutomationTests::RunTest(const FString& Parameters)
{
	TMap<FName, TArray<TSharedPtr<FTemplateItem>> > Templates = SProjectDialog::FindTemplateProjects();

	int32 OutMatchedProjectsDesk = 0;
	int32 OutCreatedProjectsDesk = 0;
	GameProjectAutomationUtils::CreateProjectSet(Templates, EHardwareClass::Desktop, EGraphicsPreset::Maximum, "Game", OutMatchedProjectsDesk, OutCreatedProjectsDesk);

	int32 OutMatchedProjectsMob = 0;
	int32 OutCreatedProjectsMob = 0;
	GameProjectAutomationUtils::CreateProjectSet(Templates, EHardwareClass::Mobile, EGraphicsPreset::Maximum, "Game", OutMatchedProjectsMob, OutCreatedProjectsMob);

	return (OutMatchedProjectsDesk == OutCreatedProjectsDesk) && (OutMatchedProjectsMob == OutCreatedProjectsMob);

}

#endif //WITH_DEV_AUTOMATION_TESTS
