// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Object.h"
#include "UObject/Interface.h"

class IClassViewerFilter;
class SWindow;
struct FSlateBrush;

/**
 * Delegate called when code is added to the project.  Passes in the created class name and class path
 * 
 * @param ClassName		The created class name
 * @param ClassPath		The created class path
 * @param ModuleName	The name of the module that the class was added to
 */
DECLARE_DELEGATE_ThreeParams( FOnAddedToProject, const FString& /*ClassName*/, const FString& /*ClassPath*/, const FString& /*ModuleName*/);

class IClassViewerFilter;

/** Information used when creating a new class via AddCodeToProject */
struct FNewClassInfo
{
	/** The type of class we want to create */
	enum class EClassType : uint8
	{
		/** The new class is using a UObject as a base, consult BaseClass for the type. */
		UObject,

		/** The new class should be an empty standard C++ class. */
		EmptyCpp,

		/** The new class should be a Slate widget, deriving from SCompoundWidget. */
		SlateWidget,

		/** The new class should be a Slate widget style, deriving from FSlateWidgetStyle, along with its associated UObject wrapper class. */
		SlateWidgetStyle,

		/** The new class is a UObject Interface, to be implemented by other UObject-based classes. */
		UInterface,
	};

	/** Default constructor; must produce an object which fails the IsSet check */
	FNewClassInfo()
		: ClassType(EClassType::UObject)
		, BaseClass(nullptr)
	{
	}

	/** Convenience constructor so you can construct from a EClassType */
	explicit FNewClassInfo(const EClassType InClassType)
		: ClassType(InClassType)
		, BaseClass(nullptr)
	{
	}

	/** Convenience constructor so you can construct from a UClass */
	explicit FNewClassInfo(const UClass* InBaseClass)
		: ClassType(EClassType::UObject)
		, BaseClass(InBaseClass)
	{
	}

	/** Check to see if this struct is set to something that could be used to create a new class */
	bool IsSet() const
	{
		return ClassType != EClassType::UObject || BaseClass;
	}

	/** Get the "friendly" class name to use in the UI */
	FText GetClassName() const;

	/** Get the class description to use in the UI */
	FText GetClassDescription(const bool bFullDescription = true) const;

	/** Get the class icon to use in the UI */
	const FSlateBrush* GetClassIcon() const;

	/** Get the C++ prefix used for this class type */
	FString GetClassPrefixCPP() const;

	/** Get the C++ class name; this may or may not be prefixed, but will always produce a valid C++ name via GetClassPrefix() + GetClassName() */
	FString GetClassNameCPP() const;

	/** Some classes may apply a particular suffix; this function returns the class name with those suffixes removed */
	FString GetCleanClassName(const FString& ClassName) const;

	/** Some classes may apply a particular suffix; this function returns the class name that will ultimately be used should that happen */
	FString GetFinalClassName(const FString& ClassName) const;

	/** Get the path needed to include this class into another file */
	bool GetIncludePath(FString& OutIncludePath) const;

	/** Gets header filename of the base class. */
	FString GetBaseClassHeaderFilename() const;

	/** Given a class name, generate the header file (.h) that should be used for this class */
	FString GetHeaderFilename(const FString& ClassName) const;

	/** Given a class name, generate the source file (.cpp) that should be used for this class */
	FString GetSourceFilename(const FString& ClassName) const;

	/** Get the generation template filename to used based on the current class type */
	FString GetHeaderTemplateFilename() const;

	/** Get the generation template filename to used based on the current class type */
	FString GetSourceTemplateFilename() const;

	/** The type of class we want to create */
	EClassType ClassType;

	/** Base class information; if the ClassType is UObject */
	const UClass* BaseClass;
};

/** Helper that creates lists of featured classes. Include FeaturedClasses.inl for definitions. */
/** @todo make this ini configurable */
struct FFeaturedClasses
{
public:
	/** Get a list of all featured native class types */
	static TArray<FNewClassInfo> AllNativeClasses();
	
	/** Get a list of all featured Actor class types */
	static TArray<FNewClassInfo> ActorClasses();

	/** Get a list of all featured Component class types */
	static TArray<FNewClassInfo> ComponentClasses();

private:
	/** Append the featured Actor class types that are commonly used */
	static void AddCommonActorClasses(TArray<FNewClassInfo>& Array);

	/** Append the featured Actor class types that are less commonly used */
	static void AddExtraActorClasses(TArray<FNewClassInfo>& Array);

	/** Append the featured Component class types that are commonly used */
	static void AddCommonComponentClasses(TArray<FNewClassInfo>& Array);

	/** Append the featured Component class types that are less commonly used */
	static void AddExtraComponentClasses(TArray<FNewClassInfo>& Array);
};

/** Add to project dialog configuration structure */
struct FAddToProjectConfig
{
	FAddToProjectConfig() : _ParentClass(nullptr), _bModal(false)	{}

public:

	/** Force the add to project dialog to use the specified parent class. Skips the first step (choose a parent class) as a result. */
	FAddToProjectConfig& ParentClass(const UClass* InClass)							{ _ParentClass = InClass; return *this; }
	/** Limits the allowable parent classes by the specified filter. */
	FAddToProjectConfig& AllowableParents(const TSharedPtr<IClassViewerFilter>& In)	{ _ParentClass = nullptr; _AllowableParents = In; return *this; }

	/** The initial path we should use as the destination for the new file, or an empty string to choose a suitable default based upon the module path. */
	FAddToProjectConfig& InitialPath(FString InInitialPath)							{ _InitialPath = MoveTemp(InInitialPath); return *this; }
	/** Optional argument that specifies the default name for the new class being added.  The user will be able to type their own name if they don't like this name.  If empty, defaults to the name of the inherited class. */
	FAddToProjectConfig& DefaultClassName(FString InDefaultClassName)				{ _DefaultClassName = MoveTemp(InDefaultClassName); return *this; }
	/** Optional argument that specifies the prefix for the new class name.  The user will be able to type their own name if they don't like this name.  Defaults to "My" if not specified or empty. */
	FAddToProjectConfig& DefaultClassPrefix(FString InDefaultClassPrefix)			{ _DefaultClassPrefix = MoveTemp(InDefaultClassPrefix); return *this; }

	/** The title text to display on the window */
	FAddToProjectConfig& WindowTitle(FText InText)									{ _WindowTitle = MoveTemp(InText); return *this; }	
	/** The parent window the dialog should use, or null to choose a suitable default parent window (the main frame, if available). */
	FAddToProjectConfig& ParentWindow(const TSharedPtr<SWindow>& InParentWindow)	{ _ParentWindow = InParentWindow; return *this; }
	/** Make the window modal to force the user to make a decision before continuing. */
	FAddToProjectConfig& Modal(bool bModal = true)									{ _bModal = bModal; return *this; }

	/** Callback for when the object is successfully added to the project */
	FAddToProjectConfig& OnAddedToProject(const FOnAddedToProject& InDelegate)		{ _OnAddedToProject = InDelegate; return *this; }

	/** Set the add to project dialog to show component types on the initial 'featured' classes list */
	FAddToProjectConfig& FeatureAllNativeClasses()									{ _FeaturedClasses = FFeaturedClasses::AllNativeClasses(); return *this; }
	/** Set the add to project dialog to show component types on the initial 'featured' classes list */
	FAddToProjectConfig& FeatureActorClasses()										{ _FeaturedClasses = FFeaturedClasses::ActorClasses(); return *this; }
	/** Set the add to project dialog to show component types on the initial 'featured' classes list */
	FAddToProjectConfig& FeatureComponentClasses()									{ _FeaturedClasses = FFeaturedClasses::ComponentClasses(); return *this; }
	/** Set whether the content browser will be synced to view the newly created class */
	FAddToProjectConfig& SyncContentBrowserToNewClass(bool bInSyncContentBrowser)	{ _SyncContentBrowserToNewClass = bInSyncContentBrowser; return *this; }

public:

	/** Forced parent class to use */
	const UClass* _ParentClass;
	/** Filter for allowable parent classes, when ParentClass is nullptr */
	TSharedPtr<IClassViewerFilter> _AllowableParents;
	/** Array of featured classes */
	TArray<FNewClassInfo> _FeaturedClasses;

	/** Initial file path for the (blueprint) class */
	FString _InitialPath;
	/** Default name prefix for the (blueprint) class */
	FString _DefaultClassPrefix;
	/** Default name for the (blueprint) class, excluding class prefix */
	FString _DefaultClassName;

	/** The title to display on the window */
	FText _WindowTitle;
	/** Parent window to use */
	TSharedPtr<SWindow> _ParentWindow;
	/** True to force a modal dialog, false otherwise */
	bool _bModal;

	/** Delegate to invoke when the (blueprint) class has been added to the project */
	FOnAddedToProject _OnAddedToProject;

	/** Whether the content browser will be synced to view the newly created class */
	bool _SyncContentBrowserToNewClass = true;
};
