// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "EditorSubsystem.h"
#include "UObject/Class.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/WeakObjectPtrTemplates.h"

#include "ClassTemplateEditorSubsystem.generated.h"

#define UE_API GAMEPROJECTGENERATION_API

// Forward Declarations
class FSubsystemCollectionBase;
class FText;
class UClass;


UCLASS(MinimalAPI, Abstract)
class UClassTemplate : public UObject
{
	GENERATED_BODY()

public:
	UE_API virtual void BeginDestroy() override;

	// Returns the directory containing the text file template for
	// the given base generated class.
	UE_API virtual FString GetDirectory() const;

	// Reads the header template text from disk.  If failure to read from
	// disk, returns false and provides text reason.
	UE_API bool ReadHeader(FString& OutHeaderFileText, FText& OutFailReason) const;

	// Reads the source template text from disk.  If failure to read from
	// disk, returns false and provides text reason.
	UE_API bool ReadSource(FString& OutSourceFileText, FText& OutFailReason) const;

	UE_API const UClass* GetGeneratedBaseClass() const;

protected:
	// Sets the generated base class associated with the given template
	UE_API void SetGeneratedBaseClass(UClass* InClass);

	// Returns the filename associated with the provided class template
	// without an extension.  Defaults to class name.
	UE_API virtual FString GetFilename() const;

	// Returns full header filename including '.h.template' extension
	UE_API FString GetHeaderFilename() const;

	// Returns full sourcefilename including '.cpp.template' extension
	UE_API FString GetSourceFilename() const;

private:
	// Base UClass of which template class corresponds.
	UPROPERTY(Transient)
	TObjectPtr<UClass> GeneratedBaseClass;
};

UCLASS(MinimalAPI, Abstract)
class UPluginClassTemplate : public UClassTemplate
{
	GENERATED_BODY()

public:
	// Returns the directory containing the text file template for
	// the given base generated class.
	UE_API virtual FString GetDirectory() const override;

protected:
	UPROPERTY(Transient)
	FString PluginName;
};

UCLASS(MinimalAPI)
class UClassTemplateEditorSubsystem : public UEditorSubsystem
{
	GENERATED_BODY()

private:
	using FTemplateRegistry = TMap<TWeakObjectPtr<const UClass>, TWeakObjectPtr<const UClassTemplate>>;
	FTemplateRegistry TemplateRegistry;

public:
	UE_API UClassTemplateEditorSubsystem();

	UE_API virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	UE_API virtual void Deinitialize() override;

	// Registers all currently loaded template classes with the internal registry.
	UE_API void RegisterTemplates();

	// Returns path to the directory containing all engine class templates.
	static UE_API FString GetEngineTemplateDirectory();

	// Returns whether or not class has registered template
	UE_API bool ContainsClassTemplate(const UClass* InClass) const;

	// Returns class template if one is registered.
	UE_API const UClassTemplate* FindClassTemplate(const UClass* InClass) const;

	friend class UClassTemplate;

private:
	// Registers a template class with the subsystem
	UE_API void Register(const UClassTemplate* InClassTemplate);

	// Unregisters a template class with the subsystem
	UE_API bool Unregister(const UClassTemplate* InClassTemplate);
};

#undef UE_API
