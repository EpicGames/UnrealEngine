// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

/** Inline file to avoid introducing many UObject headers into the global namespace */

#include "CoreTypes.h"
#include "Containers/Array.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "UObject/Interface.h"
#include "Components/ActorComponent.h"
#include "Components/SceneComponent.h"
#include "GameFramework/Actor.h"
#include "GameFramework/Pawn.h"
#include "GameFramework/Character.h"
#include "Camera/PlayerCameraManager.h"
#include "GameFramework/PlayerController.h"
#include "GameFramework/GameModeBase.h"
#include "GameFramework/WorldSettings.h"
#include "GameFramework/HUD.h"
#include "GameFramework/PlayerState.h"
#include "GameFramework/GameStateBase.h"
#include "Components/StaticMeshComponent.h"
#include "Components/SkeletalMeshComponent.h"
#include "Camera/CameraComponent.h"
#include "Components/DirectionalLightComponent.h"

struct FFeaturedClasses;
struct FNewClassInfo;

/** Get a list of all featured native class types */
FORCEINLINE TArray<FNewClassInfo> FFeaturedClasses::AllNativeClasses()
{
	TArray<FNewClassInfo> Array;
	Array.Add(FNewClassInfo(FNewClassInfo::EClassType::EmptyCpp));

	AddCommonActorClasses(Array);
	AddCommonComponentClasses(Array);

	AddExtraActorClasses(Array);

	Array.Add(FNewClassInfo(UBlueprintFunctionLibrary::StaticClass()));

	// Add the extra non-UObject classes
	Array.Add(FNewClassInfo(FNewClassInfo::EClassType::SlateWidget));
	Array.Add(FNewClassInfo(FNewClassInfo::EClassType::SlateWidgetStyle));
	Array.Add(FNewClassInfo(FNewClassInfo::EClassType::UInterface));
	return Array;
}

/** Get a list of all featured Actor class types */
FORCEINLINE TArray<FNewClassInfo> FFeaturedClasses::ActorClasses()
{
	TArray<FNewClassInfo> Array;
	AddCommonActorClasses(Array);
	AddExtraActorClasses(Array);
	return Array;
}

/** Get a list of all featured Component class types */
FORCEINLINE TArray<FNewClassInfo> FFeaturedClasses::ComponentClasses()
{
	TArray<FNewClassInfo> Array;
	AddCommonComponentClasses(Array);
	AddExtraComponentClasses(Array);
	return Array;
}

/** Append the featured Actor class types that are commonly used */
FORCEINLINE void FFeaturedClasses::AddCommonActorClasses(TArray<FNewClassInfo>& Array)
{
	Array.Add(FNewClassInfo(ACharacter::StaticClass()));
	Array.Add(FNewClassInfo(APawn::StaticClass()));
	Array.Add(FNewClassInfo(AActor::StaticClass()));
}

/** Append the featured Actor class types that are less commonly used */
FORCEINLINE void FFeaturedClasses::AddExtraActorClasses(TArray<FNewClassInfo>& Array)
{
	Array.Add(FNewClassInfo(APlayerCameraManager::StaticClass()));
	Array.Add(FNewClassInfo(APlayerController::StaticClass()));
	Array.Add(FNewClassInfo(AGameModeBase::StaticClass()));
	Array.Add(FNewClassInfo(AWorldSettings::StaticClass()));
	Array.Add(FNewClassInfo(AHUD::StaticClass()));
	Array.Add(FNewClassInfo(APlayerState::StaticClass()));
	Array.Add(FNewClassInfo(AGameStateBase::StaticClass()));
}

/** Append the featured Component class types that are commonly used */
FORCEINLINE void FFeaturedClasses::AddCommonComponentClasses(TArray<FNewClassInfo>& Array)
{
	Array.Add(FNewClassInfo(UActorComponent::StaticClass()));
	Array.Add(FNewClassInfo(USceneComponent::StaticClass()));
}

/** Append the featured Component class types that are less commonly used */
FORCEINLINE void FFeaturedClasses::AddExtraComponentClasses(TArray<FNewClassInfo>& Array)
{
	Array.Add(FNewClassInfo(UStaticMeshComponent::StaticClass()));
	Array.Add(FNewClassInfo(USkeletalMeshComponent::StaticClass()));
	Array.Add(FNewClassInfo(UCameraComponent::StaticClass()));
	Array.Add(FNewClassInfo(UDirectionalLightComponent::StaticClass()));
}
