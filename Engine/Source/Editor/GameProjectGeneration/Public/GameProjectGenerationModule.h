// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "AddToProjectConfig.h"
#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "Delegates/Delegate.h"
#include "HAL/Platform.h"
#include "ModuleDescriptor.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"

class FText;
class UClass;
struct FModuleContextInfo;
struct FProjectInformation;
struct FSlateBrush;
struct FTemplateCategory;

/**
 * Game Project Generation module
 */
class FGameProjectGenerationModule : public IModuleInterface
{

public:
	typedef TMap<FName, TSharedPtr<FTemplateCategory>> FTemplateCategoryMap;

	/**
	 * Called right after the plugin DLL has been loaded and the plugin object has been created
	 */
	virtual void StartupModule();

	/**
	 * Called before the plugin is unloaded, right before the plugin object is destroyed.
	 */
	virtual void ShutdownModule();

	/**
	 * Singleton-like access to this module's interface.  This is just for convenience!
	 * Beware of calling this during the shutdown phase, though.  Your module might have been unloaded already.
	 *
	 * @return Returns singleton instance, loading the module on demand if needed
	 */
	static inline FGameProjectGenerationModule& Get()
	{
		static const FName ModuleName = "GameProjectGeneration";
		return FModuleManager::LoadModuleChecked< FGameProjectGenerationModule >( ModuleName );
	}

	/** Creates the game project dialog */
	virtual TSharedRef<class SWidget> CreateGameProjectDialog(bool bAllowProjectOpening, bool bAllowProjectCreate);

	/** Creates a new class dialog for creating classes based on the passed-in class. */
	virtual TSharedRef<class SWidget> CreateNewClassDialog(const UClass* InClass);

	/** Creates the project browser widget */
	UE_DEPRECATED(5.8, "Use CreateEmbeddedProjectBrowser")
	virtual TSharedRef<class SWidget> CreateProjectBrowser(bool bInShowHeaderBar = true, bool bInAllowScrolling = true);

	/** Creates the embedded project browser widget */
	virtual TSharedRef<class SWidget> CreateEmbeddedProjectBrowser();

	/** 
	 * Opens a dialog to add code files to the current project. 
	 *
	 * @param	Config		Dialog configuration options
	 */
	virtual void OpenAddCodeToProjectDialog(const FAddToProjectConfig& Config = FAddToProjectConfig());

	/** 
	 * Opens a dialog to add a new blueprint to the current project. 
	 *
	 * @param	Config		Dialog configuration options
	 */
	virtual void OpenAddBlueprintToProjectDialog(const FAddToProjectConfig& Config);
	
	/** Delegate for when the AddCodeToProject dialog is opened */
	DECLARE_EVENT(FGameProjectGenerationModule, FAddCodeToProjectDialogOpenedEvent);
	FAddCodeToProjectDialogOpenedEvent& OnAddCodeToProjectDialogOpened() { return AddCodeToProjectDialogOpenedEvent; }

	/** Tries to make the project file writable. Prompts to check out as necessary. */
	virtual void TryMakeProjectFileWriteable(const FString& ProjectFile);

	/** Prompts the user to update their project file, if necessary. */
	virtual void CheckForOutOfDateGameProjectFile();

	/** Updates the currently loaded project. Returns true if the project was updated successfully or if no update was needed */
	virtual bool UpdateGameProject(const FString& ProjectFile, const FString& EngineIdentifier, FText& OutFailReason);

	/** Updates the current code project */
	virtual bool UpdateCodeProject(FText& OutFailReason, FText& OutFailLog);

	/** Gets the current projects source file count */
	virtual bool ProjectHasCodeFiles();

	/** Returns the path to the module's include header */
	virtual FString DetermineModuleIncludePath(const FModuleContextInfo& ModuleInfo, const FString& FileRelativeTo);

	/** Get the information about any modules referenced in the .uproject file of the currently loaded project */
	virtual const TArray<FModuleContextInfo>& GetCurrentProjectModules();

	/** Returns true if the specified class is a valid base class for the given module */
	virtual bool IsValidBaseClassForCreation(const UClass* InClass, const FModuleContextInfo& InModuleInfo);

	/** Returns true if the specified class is a valid base class for any of the given modules */
	virtual bool IsValidBaseClassForCreation(const UClass* InClass, const TArray<FModuleContextInfo>& InModuleInfoArray);

	/** Gets file and size info about the source directory */
	virtual void GetProjectSourceDirectoryInfo(int32& OutNumFiles, int64& OutDirectorySize);

	/** Warn the user if the project filename is invalid in case they renamed it outside the editor */
	virtual void CheckAndWarnProjectFilenameValid();

	/** Generate basic project source code */
	virtual bool GenerateBasicSourceCode(TArray<FString>& OutCreatedFiles, FText& OutFailReason);

	/**
	 * Update the list of supported target platforms based upon the parameters provided
	 * This will take care of checking out and saving the updated .uproject file automatically
	 * 
	 * @param	InPlatformName		Name of the platform to target (eg, WindowsClient)
	 * @param	bIsSupported		true if the platform should be supported by this project, false if it should not
	 */
	virtual void UpdateSupportedTargetPlatforms(const FName& InPlatformName, const bool bIsSupported);

	/** Clear the list of supported target platforms */
	virtual void ClearSupportedTargetPlatforms();

public:

	/** Creates the specified project file and all required folders. If TemplateFile is non-empty, it will be used as the template for creation. On failure, OutFailReason will be populated. */
	virtual bool CreateProject(const FProjectInformation& InProjectInfo, FText& OutFailReason, FText& OutFailLog, TArray<FString>* OutCreatedFiles = nullptr);

	/** Opens the specified project, if it exists. Returns true if the project file is valid. On failure, OutFailReason will be populated. */
	virtual bool OpenProject(const FString& InProjectFile, FText& OutFailReason);

	// Non DLL-exposed access to template categories
	TSharedPtr<const FTemplateCategory> GetCategory(FName Type) const { return TemplateCategories.FindRef(Type); }

	void GetAllTemplateCategories(TArray<TSharedPtr<FTemplateCategory>>& OutCategories) const { TemplateCategories.GenerateValueArray(OutCategories); }

private:
	FAddCodeToProjectDialogOpenedEvent AddCodeToProjectDialogOpenedEvent;

	/** Map of template categories from type to ptr */
	FTemplateCategoryMap TemplateCategories;

	void LoadTemplateCategories();
};
