// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AddToProjectConfig.h"
#include "Containers/Array.h"
#include "Containers/Set.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "Delegates/Delegate.h"
#include "GameProjectGenerationModule.h"
#include "HAL/Platform.h"
#include "HardwareTargetingSettings.h"
#include "Misc/Optional.h"
#include "SlateFwd.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"

#define UE_API GAMEPROJECTGENERATION_API

class FText;
class SNotificationItem;
class UClass;
class UTemplateCategories;
class UTemplateProjectDefs;
struct FAddToProjectConfig;
struct FGuid;
struct FModuleContextInfo;
struct FNewClassInfo;
struct FProjectDescriptor;

enum class EClassDomain : uint8;
struct FTemplateConfigValue;

struct FProjectInformation
{
	FProjectInformation() = default;

	FString ProjectFilename;
	FString TemplateFile;
	FName TemplateCategory;
	FName Variant;

	bool bShouldGenerateCode = false;
	bool bIsBlankTemplate = false;
	bool bIsEnterpriseProject = false;

	// These are all optional, because there is an additional state introduced by hiding the setting in the template.
	// In this case, the template author has chosen not to give the user a choice,
	// so we must assume that the template already controls these settings explicitly.
	TOptional<bool> bEnableXR;

	TOptional<EHardwareClass> TargetedHardware;
	TOptional<EGraphicsPreset> DefaultGraphicsPerformance;
};

DECLARE_DELEGATE_RetVal_OneParam(bool, FProjectDescriptorModifier, FProjectDescriptor&);

class GameProjectUtils
{
public:
	/** Where is this class located within the Source folder? */
	enum class EClassLocation : uint8
	{
		/** The class is going to a user defined location (outside of the Public, Private, or Classes) folder for this module */
		UserDefined,

		/** The class is going to the Public folder for this module */
		Public,

		/** The class is going to the Private folder for this module */
		Private,

		/** The class is going to the Classes folder for this module */
		Classes,
	};

	/** Used as a function return result when adding new code to the project */
	enum class EAddCodeToProjectResult : uint8
	{
		/** Function has successfully added the code and hot-reloaded the required module(s) */
		Succeeded,

		/** There were errors with the input given to the function */
		InvalidInput,

		/** There were errors when adding the new source files */
		FailedToAddCode,

		/** There were errors when hot-reloading the new module */
		FailedToHotReload,
	};

	/** Used as a function return result when a project is duplicated when upgrading project's version in Convert project dialog - Open a copy */
	enum class EProjectDuplicateResult : uint8
	{
		/** Function has successfully duplicated all project files */
		Succeeded,

		/** There were errors while duplicating project files */
		Failed,

		/** User has canceled project duplication process */
		UserCanceled
	};

	/** Reload status when adding new code to the project */
	enum class EReloadStatus : uint8
	{
		/** Code was built and reloaded */
		Reloaded,

		/** Code was not reloaded */
		NotReloaded,
	};

	/** Returns true if the project filename is properly formed and does not conflict with another project */
	static UE_API bool IsValidProjectFileForCreation(const FString& ProjectFile, FText& OutFailReason);

	/** Opens the specified project, if it exists. Returns true if the project file is valid. On failure, OutFailReason will be populated. */
	static UE_API bool OpenProject(const FString& ProjectFile, FText& OutFailReason);

	/** Opens the code editing IDE for the specified project, if it exists. Returns true if the IDE could be opened. On failure, OutFailReason will be populated. */
	static UE_API bool OpenCodeIDE(const FString& ProjectFile, FText& OutFailReason);

	/** Creates the specified project file and all required folders. If TemplateFile is non-empty, it will be used as the template for creation. On failure, OutFailReason will be populated. */
	static UE_API bool CreateProject(const FProjectInformation& InProjectInfo, FText& OutFailReason, FText& OutFailLog, TArray<FString>* OutCreatedFiles = nullptr);

	/** Prompts the user to update their project file, if necessary. */
	static UE_API void CheckForOutOfDateGameProjectFile();

	/** Warn the user if the project filename is invalid in case they renamed it outside the editor */
	static UE_API void CheckAndWarnProjectFilenameValid();

	/** Checks out the current project file (or prompts to make writable) */
	static UE_API void TryMakeProjectFileWriteable(const FString& ProjectFile);

	/** Updates the given project file to an engine identifier. Returns true if the project was updated successfully or if no update was needed */
	static UE_API bool UpdateGameProject(const FString& ProjectFile, const FString& EngineIdentifier, FText& OutFailReason);

	/**
	 * Opens a dialog to add code files or blueprints to the current project.
	 *
	 * @param	Config			Configuration options for the dialog
	 * @param	InDomain		The domain of the class we're creating (native or blueprint)
	 */
	static UE_API void OpenAddToProjectDialog(const FAddToProjectConfig& Config, EClassDomain InDomain);

	/** Returns true if the specified class name is properly formed and does not conflict with another class */
	static UE_API bool IsValidClassNameForCreation(const FString& NewClassName, FText& OutFailReason);

	/** Returns true if the specified class name is properly formed and does not conflict with another class, including source/header files */
	static UE_API bool IsValidClassNameForCreation(const FString& NewClassName, const FModuleContextInfo& ModuleInfo, const TSet<FString>& DisallowedHeaderNames, FText& OutFailReason);

	/** Returns true if the specified class is a valid base class for the given module */
	static UE_API bool IsValidBaseClassForCreation(const UClass* InClass, const FModuleContextInfo& InModuleInfo);

	/** Returns true if the specified class is a valid base class for any of the given modules */
	static UE_API bool IsValidBaseClassForCreation(const UClass* InClass, const TArray<FModuleContextInfo>& InModuleInfoArray);

	/** Adds new source code to the project. When returning Succeeded or FailedToHotReload, OutSyncFileAndLineNumber will be the the preferred target file to sync in the users code editing IDE, formatted for use with GenericApplication::GotoLineInSource */
	static UE_API EAddCodeToProjectResult AddCodeToProject(const FString& NewClassName, const FString& NewClassPath, const FModuleContextInfo& ModuleInfo, const FNewClassInfo ParentClassInfo, const TSet<FString>& DisallowedHeaderNames, FString& OutHeaderFilePath, FString& OutCppFilePath, FText& OutFailReason);

	/** Adds new source code to the project. When returning Succeeded or FailedToHotReload, OutSyncFileAndLineNumber will be the the preferred target file to sync in the users code editing IDE, formatted for use with GenericApplication::GotoLineInSource */
	static UE_API EAddCodeToProjectResult AddCodeToProject(const FString& NewClassName, const FString& NewClassPath, const FModuleContextInfo& ModuleInfo, const FNewClassInfo ParentClassInfo, const TSet<FString>& DisallowedHeaderNames, FString& OutHeaderFilePath, FString& OutCppFilePath, FText& OutFailReason, EReloadStatus& OutReloadStatus);

	/** Loads a list of template categories defined in the TemplateCategories.ini file in the specified folder */
	static UE_API UTemplateCategories* LoadTemplateCategories(const FString& RootDir);

	/** Loads a template project definitions object from the TemplateDefs.ini file in the specified project */
	static UE_API UTemplateProjectDefs* LoadTemplateDefs(const FString& ProjectDirectory);

	/** @return The number of code files in the currently loaded project */
	static UE_API int32 GetProjectCodeFileCount();

	/**
	* Retrieves file and size info about the project's source directory
	* @param OutNumFiles Contains the number of files within the source directory
	* @param OutDirectorySize Contains the combined size of all files in the directory
	*/
	static UE_API void GetProjectSourceDirectoryInfo(int32& OutNumFiles, int64& OutDirectorySize);

	/** Returns the uproject template filename for the default project template. */
	static UE_API FString GetDefaultProjectTemplateFilename();

	/** Compiles a project while showing a progress bar, and offers to open the IDE if it fails. */
	static UE_API bool BuildCodeProject(const FString& ProjectFilename);

	/** Creates code project files for a new game project. On failure, OutFailReason and OutFailLog will be populated. */
	static UE_API bool GenerateCodeProjectFiles(const FString& ProjectFilename, FText& OutFailReason, FText& OutFailLog);

	/** Returns true if there are engine starter content files available for instancing into new projects. */
	UE_DEPRECATED(5.6, "Ability to add Starter Content has been removed")
	static bool IsEngineStarterContentAvailable() { return false; }

	/** Returns true if the given project is referencing engine starter content pack. */
	UE_DEPRECATED(5.6, "Ability to add Starter Content has been removed")
	static bool IsUsingEngineStarterContent(const FProjectInformation& ProjectInfo) { return false; }

	/** Returns true if there are any starter content packs are available for given project. */
	UE_DEPRECATED(5.6, "Ability to add Starter Content has been removed")
	static bool IsStarterContentAvailableForProject(const FProjectInformation& ProjectInfo) { return false; }

	/**
	 * Get the information about any modules referenced in the .uproject file of the currently loaded project
	 */
	static UE_API const TArray<FModuleContextInfo>& GetCurrentProjectModules();

	/**
	* Get the information about any modules in any of the plugins in the currently loaded project (Ignores Engine Plugins)
	*/
	static UE_API TArray<FModuleContextInfo> GetCurrentProjectPluginModules();

	/**
	 * Reset the cached result of information about any modules referenced in the .uproject file of the currently loaded project
	 */
	static UE_API void ResetCurrentProjectModulesCache();

	/**
	 * Check to see if the given path is a valid place to put source code for this project (exists within the source root path)
	 *
	 * @param	InPath				The path to check
	 * @param	ModuleInfo			Information about the module being validated
	 * @param	OutFailReason		Optional parameter to fill with failure information
	 *
	 * @return	true if the path is valid, false otherwise
	 */
	static UE_API bool IsValidSourcePath(const FString& InPath, const FModuleContextInfo& ModuleInfo, FText* const OutFailReason = nullptr);

	/**
	 * Given the path provided, work out where generated .h and .cpp files would be placed
	 *
	 * @param	InPath				The path to use a base
	 * @param	ModuleInfo			Information about the module being validated
	 * @param	OutHeaderPath		The path where the .h file should be placed
	 * @param	OutSourcePath		The path where the .cpp file should be placed
	 * @param	OutFailReason		Optional parameter to fill with failure information
	 *
	 * @return	false if the paths are invalid
	 */
	static UE_API bool CalculateSourcePaths(const FString& InPath, const FModuleContextInfo& ModuleInfo, FString& OutHeaderPath, FString& OutSourcePath, FText* const OutFailReason = nullptr);

	/**
	 * Given the path provided, work out where it's located within the Source folder
	 *
	 * @param	InPath				The path to use a base
	 * @param	ModuleInfo			Information about the module being validated
	 * @param	OutClassLocation	The location within the Source folder
	 * @param	OutFailReason		Optional parameter to fill with failure information
	 *
	 * @return	false if the paths are invalid
	 */
	static UE_API bool GetClassLocation(const FString& InPath, const FModuleContextInfo& ModuleInfo, EClassLocation& OutClassLocation, FText* const OutFailReason = nullptr);

	/** Creates a copy of a project directory in order to upgrade it. */
	static UE_API EProjectDuplicateResult DuplicateProjectForUpgrade( const FString& InProjectFile, FString& OutNewProjectFile );

	/**
	 * Update the list of supported target platforms based upon the parameters provided
	 * This will take care of checking out and saving the updated .uproject file automatically
	 *
	 * @param	InPlatformName		Name of the platform to target (eg, WindowsClient)
	 * @param	bIsSupported		true if the platform should be supported by this project, false if it should not
	 */
	static UE_API void UpdateSupportedTargetPlatforms(const FName& InPlatformName, const bool bIsSupported);

	/** Clear the list of supported target platforms */
	static UE_API void ClearSupportedTargetPlatforms();

	/** Returns the path to the module's include header */
	static UE_API FString DetermineModuleIncludePath(const FModuleContextInfo& ModuleInfo, const FString& FileRelativeTo);

	/** Creates the basic source code for a new project. On failure, OutFailReason will be populated. */
	static UE_API bool GenerateBasicSourceCode(TArray<FString>& OutCreatedFiles, FText& OutFailReason);

	/** Generates a Build.cs file for a game module */
	static UE_API bool GenerateGameModuleBuildFile(const FString& NewBuildFileName, const FString& ModuleName, const TArray<FString>& PublicDependencyModuleNames, const TArray<FString>& PrivateDependencyModuleNames, FText& OutFailReason);

	/** Generates a Build.cs file for a plugin module. Set 'bUseExplicitOrSharedPCHs' to false to disable IWYU conventions. */
	static UE_API bool GeneratePluginModuleBuildFile(const FString& NewBuildFileName, const FString& ModuleName, const TArray<FString>& PublicDependencyModuleNames, const TArray<FString>& PrivateDependencyModuleNames, FText& OutFailReason, bool bUseExplicitOrSharedPCHs = true);

	/** Generates a module .cpp file, intended for plugin use */
	static UE_API bool GeneratePluginModuleCPPFile(const FString& CPPFileName, const FString& ModuleName, const FString& StartupSourceCode, FText& OutFailReason);

	/** Generates a module .h file, intended for plugin use */
	static UE_API bool GeneratePluginModuleHeaderFile(const FString& HeaderFileName, const TArray<FString>& PublicHeaderIncludes, FText& OutFailReason);

	/** Returns true if the currently loaded project has code files */
	static UE_API bool ProjectHasCodeFiles();

	/** Returns the contents of the specified template file */
	static UE_API bool ReadTemplateFile(const FString& TemplateFileName, FString& OutFileContents, FText& OutFailReason);

	/** Writes an output file. OutputFilename includes a path */
	static UE_API bool WriteOutputFile(const FString& OutputFilename, const FString& OutputFileContents, FText& OutFailReason);

	/** Returns a comma delimited string comprised of all the elements in InList. If bPlaceQuotesAroundEveryElement, every element is within quotes. */
	static UE_API FString MakeCommaDelimitedList(const TArray<FString>& InList, bool bPlaceQuotesAroundEveryElement = true);

	/** Checks the name for illegal characters */
	static UE_API bool NameContainsOnlyLegalCharacters(const FString& TestName, FString& OutIllegalCharacters);

	/** Checks the name to see if it matches platform module names */
	static UE_API bool NameMatchesPlatformModuleName(const FString& TestName);

	/** Returns a list of #include lines formed from InList */
	static UE_API FString MakeIncludeList(const TArray<FString>& InList);

	/** Deletes the specified list of files that were created during file creation */
	static UE_API void DeleteCreatedFiles(const FString& RootFolder, const TArray<FString>& CreatedFiles);

	/**
	 * Update the list of plugin directories to scan
	 * This will take care of checking out and saving the updated .uproject file automatically
	 *
	 * @param	InDir directory to add/remove
	 * @param	bAddOrRemove true if the directory should be added to this project, false if it should not
	 * @return Whether the plugin directory list was changed
	 */
	static UE_API bool UpdateAdditionalPluginDirectory(const FString& InDir, const bool bAddOrRemove);

	/** Gets the default build settings version for UBT */
	static UE_API const TCHAR* GetDefaultBuildSettingsVersion();

private:
	
	/** Add hardware-specific config values such as the target platform and RHI. */
	static UE_API void AddHardwareConfigValues(const FProjectInformation& InProjectInfo, TArray<FTemplateConfigValue>& ConfigValues);
	
	/** Get the name of the starter content pack to use for the given project. */
	UE_DEPRECATED(5.6, "Ability to add Starter Content has been removed")
	static FString GetStarterContentName(const FProjectInformation& InProjectInfo) { return FString{}; }

	/** Generates a new project without using a template project */
	static UE_API TOptional<FGuid> GenerateProjectFromScratch(const FProjectInformation& InProjectInfo, FText& OutFailReason, FText& OutFailLog);

	/** Generates a new project using a template project */
	static UE_API TOptional<FGuid> CreateProjectFromTemplate(const FProjectInformation& InProjectInfo, FText& OutFailReason, FText& OutFailLog, TArray<FString>* OutCreatedFiles = nullptr);

	/** Sets the engine association for a new project. Handles foreign and non-foreign projects. */
	static UE_API bool SetEngineAssociationForForeignProject(const FString& ProjectFileName, FText& OutFailReason);

	/** Insert any required feature packs into the DefaultGame.ini file */
	UE_DEPRECATED(5.6, "Ability to add Starter Content has been removed")
	static UE_API bool InsertFeaturePacksIntoINIFile(const FProjectInformation& InProjectInfo, FText& OutFailReason);

	/*
	 * Insert the addition files from any feature packs specified in the temapalte defs file
	 * @param InProjectInfo		Project infor to add content for
	 * @param CreatedFiles		List of files we copied
	 * @param OutFailReason		Failure reason (if any)
	 *
	 * @returns true if no errors
	 */
	static UE_API bool AddSharedContentToProject(const FProjectInformation &InProjectInfo, TArray<FString> &CreatedFiles, FText& OutFailReason);

	/** Returns the template defs ini filename */
	static UE_API FString GetTemplateDefsFilename();

	/** Returns the include header path for a given fully specified, normalized file path */
	static UE_API FString GetIncludePathForFile(const FString& InFullFilePath, const FString& ModuleRootPath);

	/** Returns true if the project file exists on disk */
	static UE_API bool ProjectFileExists(const FString& ProjectFile);

	/** Returns true if any project files exist in the given folder */
	static UE_API bool AnyProjectFilesExistInFolder(const FString& Path);

	/** Returns true if file cleanup on failure is enabled, false if not */
	static UE_API bool CleanupIsEnabled();

	/** Creates ini files for a new project. On failure, OutFailReason will be populated. */
	static UE_API bool GenerateConfigFiles(const FProjectInformation& InProjectInfo, TArray<FString>& OutCreatedFiles, FText& OutFailReason, FGuid& OutProjectID);

	/* Creates new ini files for a specific project's platform configurations. */
	static UE_API bool GeneratePlatformConfigFiles(const FProjectInformation& InProjectInfo, FText& OutFailReason);

	/** Creates the basic source code for a new project. On failure, OutFailReason will be populated. */
	static UE_API bool GenerateBasicSourceCode(const FString& NewProjectSourcePath, const FString& NewProjectName, const FString& NewProjectRoot, TArray<FString>& OutGeneratedStartupModuleNames, TArray<FString>& OutCreatedFiles, FText& OutFailReason);

	/** Creates the game framework source code for a new project (Pawn, GameMode, PlayerControlleR). On failure, OutFailReason will be populated. */
	static UE_API bool GenerateGameFrameworkSourceCode(const FString& NewProjectSourcePath, const FString& NewProjectName, TArray<FString>& OutCreatedFiles, FText& OutFailReason);

	/** Creates the batch file to regenerate code projects. */
	static UE_API bool GenerateCodeProjectGenerationBatchFile(const FString& ProjectFolder, TArray<FString>& OutCreatedFiles, FText& OutFailReason);

	/** Creates the batch file for launching the editor or game */
	static UE_API bool GenerateLaunchBatchFile(const FString& ProjectName, const FString& ProjectFolder, bool bLaunchEditor, TArray<FString>& OutCreatedFiles, FText& OutFailReason);

	/** Returns the copyright line used at the top of all files */
	static UE_API FString MakeCopyrightLine();

	/** Generates a header file for a UObject class. OutSyncLocation is a string representing the preferred cursor sync location for this file after creation. */
	static UE_API bool GenerateClassHeaderFile(const FString& NewHeaderFileName, const FString UnPrefixedClassName, const FNewClassInfo ParentClassInfo, const TArray<FString>& ClassSpecifierList, const FString& ClassProperties, const FString& ClassFunctionDeclarations, FString& OutSyncLocation, const FModuleContextInfo& ModuleInfo, bool bDeclareConstructor, FText& OutFailReason);

	/** Finds the cursor sync location in the source file and reports it back as a string */
	static UE_API void HarvestCursorSyncLocation( FString& FinalOutput, FString& OutSyncLocation );

	/** Generates a cpp file for a UObject class */
	static UE_API bool GenerateClassCPPFile(const FString& NewCPPFileName, const FString UnPrefixedClassName, const FNewClassInfo ParentClassInfo, const TArray<FString>& AdditionalIncludes, const TArray<FString>& PropertyOverrides, const FString& AdditionalMemberDefinitions, FString& OutSyncLocation, const FModuleContextInfo& ModuleInfo, FText& OutFailReason);

	/** Generates a Target.cs file for a game module */
	static UE_API bool GenerateGameModuleTargetFile(const FString& NewTargetFileName, const FString& ModuleName, const TArray<FString>& ExtraModuleNames, FText& OutFailReason);

	/** Generates a Build.cs file for a Editor module */
	static UE_API bool GenerateEditorModuleBuildFile(const FString& NewBuildFileName, const FString& ModuleName, const TArray<FString>& PublicDependencyModuleNames, const TArray<FString>& PrivateDependencyModuleNames, FText& OutFailReason);

	/** Generates a Target.cs file for a Editor module */
	static UE_API bool GenerateEditorModuleTargetFile(const FString& NewTargetFileName, const FString& ModuleName, const TArray<FString>& ExtraModuleNames, FText& OutFailReason);

	/** Generates a main game module cpp file */
	static UE_API bool GenerateGameModuleCPPFile(const FString& NewGameModuleCPPFileName, const FString& ModuleName, const FString& GameName, FText& OutFailReason);

	/** Generates a main game module header file */
	static UE_API bool GenerateGameModuleHeaderFile(const FString& NewGameModuleHeaderFileName, const TArray<FString>& PublicHeaderIncludes, FText& OutFailReason);

	/**
	* Replace a wildcard with another string
	*
	* @param Input				The input string
	* @param From				The wildcard to be replaced
	* @param To					The text the wildcard should be replaced with
	* @param bLeadingTab		If the line where the wildcard is located starts with a tab
	* @param bTrailingNewLine	If the line where the wildcard is located ends with a new line

	* @return the input string after replacement of the wildcard
	*/
	static UE_API FString ReplaceWildcard(const FString& Input, const FString& From, const FString& To, bool bLeadingTab = false, bool bTrailingNewLine = false);

	/** Handler for when the user confirms a project update */
	static UE_API void OnUpdateProjectConfirm();

	/** @param OutProjectCodeFilenames Contains the filenames of the project source code files */
	static UE_API void GetProjectCodeFilenames(TArray<FString>& OutProjectCodeFilenames);

	/**
	 * Updates the projects and modifies FProjectDescriptor accordingly to given modifier.
	 *
	 * @param Modifier	Callback delegate that will modify the project descriptor accordingly.
	 */
	static UE_API void UpdateProject(const FProjectDescriptorModifier& Modifier);

	/**
	 * Updates the project file.
	 */
	static UE_API void UpdateProject();

	/**
	 * Updates the projects, and optionally the modules names
	 *
	 * @param StartupModuleNames	if specified, replaces the existing module names with this version
	 */
	static UE_API void UpdateProject(const TArray<FString>* StartupModuleNames);

	/** Handler for when the user opts out of a project update */
	static UE_API void OnUpdateProjectCancel();

	/**
	 * Updates the loaded game project file to the current version and to use the given modules
	 *
	 * @param ProjectFilename		The name of the project (used to checkout from source control)
	 * @param EngineIdentifier		The identifier for the engine to open the project with
	 * @param StartupModuleNames	if specified, replaces the existing module names with this version
	 * @param OutFailReason			Out, if unsuccessful this is the reason why

	 * @return true, if successful
	 */
	static UE_API bool UpdateGameProjectFile(const FString& ProjectFilename, const FString& EngineIdentifier, const TArray<FString>* StartupModuleNames, FText& OutFailReason);

	/**
	 * Updates the loaded game project file to the current version and modifies FProjectDescriptor accordingly to given modifier.
	 *
	 * @param ProjectFilename		The name of the project (used to checkout from source control)
	 * @param EngineIdentifier		The identifier for the engine to open the project with
	 * @param Modifier				Callback delegate that will modify the project descriptor accordingly.
	 * @param OutFailReason			Out, if unsuccessful this is the reason why

	 * @return true, if successful
	 */
	static UE_API bool UpdateGameProjectFile(const FString& ProjectFilename, const FString& EngineIdentifier, const FProjectDescriptorModifier& Modifier, FText& OutFailReason);

	/**
	 * Updates the loaded game project file to the current version.
	 *
	 * @param ProjectFilename		The name of the project (used to checkout from source control)
	 * @param EngineIdentifier		The identifier for the engine to open the project with
	 * @param OutFailReason			Out, if unsuccessful this is the reason why

	 * @return true, if successful
	 */
	static UE_API bool UpdateGameProjectFile(const FString& ProjectFilename, const FString& EngineIdentifier, FText& OutFailReason);

	/** Checks the specified game project file out from source control */
	static UE_API bool CheckoutGameProjectFile(const FString& ProjectFilename, FText& OutFailReason);

	/** Internal handler for AddCodeToProject*/
	static UE_API EAddCodeToProjectResult AddCodeToProject_Internal(const FString& NewClassName, const FString& NewClassPath, const FModuleContextInfo& ModuleInfo, const FNewClassInfo ParentClassInfo, const TSet<FString>& DisallowedHeaderNames, FString& OutHeaderFilePath, FString& OutCppFilePath, FText& OutFailReason, EReloadStatus& OutReloadStatus);

	/** Internal handler for IsValidBaseClassForCreation */
	DECLARE_DELEGATE_RetVal_OneParam(bool, FDoesClassNeedAPIExportCallback, const FString& /*ClassModuleName*/);
	static UE_API bool IsValidBaseClassForCreation_Internal(const UClass* InClass, const FDoesClassNeedAPIExportCallback& InDoesClassNeedAPIExport);

	/** Handler for the user confirming they've read the name legnth warning */
	static UE_API void OnWarningReasonOk();

	/** Given a source file name, find its location within the project */
	static UE_API bool FindSourceFileInProject(const FString& InFilename, const FString& InSearchPath, FString& OutPath);

	/**
	 * Gets required additional dependencies for fresh project for given class info.
	 *
	 * @param ClassInfo Given class info.
	 *
	 * @returns Array of required dependencies.
	 */
	static UE_API TArray<FString> GetRequiredAdditionalDependencies(const FNewClassInfo& ClassInfo);

	/**
	 * Updates startup module names in project descriptor.
	 *
	 * @param Descriptor Descriptor to update.
	 * @param StartupModuleNames Modules to fill.
	 *
	 * @returns True if descriptor has been modified. False otherwise.
	 */
	static UE_API bool UpdateStartupModuleNames(FProjectDescriptor& Descriptor, const TArray<FString>* StartupModuleNames);

	/**
	 * Updates additional dependencies in project descriptor.
	 *
	 * @param Descriptor Descriptor to update.
	 * @param RequiredDependencies Required dependencies.
	 * @param ModuleName Module name for which those dependencies are required.
	 *
	 * @returns True if descriptor has been modified. False otherwise.
	 */
	static UE_API bool UpdateRequiredAdditionalDependencies(FProjectDescriptor& Descriptor, TArray<FString>& RequiredDependencies, const FString& ModuleName);

	/**
	 * Updates the projects and modifies FProjectDescriptor accordingly to given modifier.
	 *
	 * @param Modifier	Callback delegate that will modify the project descriptor accordingly.
	 */
	static UE_API void UpdateProject_Impl(const FProjectDescriptorModifier* Modifier);

	/**
	 * Updates the loaded game project file to the current version and modifies FProjectDescriptor accordingly to given modifier.
	 *
	 * @param ProjectFilename		The name of the project (used to checkout from source control)
	 * @param EngineIdentifier		The identifier for the engine to open the project with
	 * @param Modifier				Callback delegate that will modify the project descriptor accordingly.
	 * @param OutFailReason			Out, if unsuccessful this is the reason why

	 * @return true, if successful
	 */
	static UE_API bool UpdateGameProjectFile_Impl(const FString& ProjectFilename, const FString& EngineIdentifier, const FProjectDescriptorModifier* Modifier, FText& OutFailReason);

private:

	static UE_API TWeakPtr<SNotificationItem> UpdateGameProjectNotification;
	static UE_API TWeakPtr<SNotificationItem> WarningProjectNameNotification;

	// Whether we should use AudioMixer for all platforms:
	static UE_API bool bUseAudioMixerForAllPlatforms;

	constexpr static const TCHAR IncludePathFormatString[] = TEXT("#include \"%s\"");
};

#undef UE_API
